// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassArchetypeTypes.h"
#include "MassEntityTypes.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FArchetype_VersionedHandleInvalidation : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);

		// Get a versioned handle
		FMassArchetypeVersionedHandle VersionedHandle(IntsArchetype);

		AITEST_TRUE("Versioned handle is valid initially", VersionedHandle.IsValid());

		// Mutate entities — destroying should change entity order version
		EntityManager->DestroyEntity(Entities[0]);
		EntityManager->DestroyEntity(Entities[5]);

		// The versioned handle should now be out of date (entity order changed)
		AITEST_FALSE("Versioned handle is NOT up to date after entity destruction", VersionedHandle.IsUpToDate());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetype_VersionedHandleInvalidation, "System.Mass.Coverage.Archetype.VersionedHandleInvalidation");

struct FArchetype_SharedFragmentPerChunkStorage : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create archetype with fragment + const shared fragment type
		FMassArchetypeCompositionDescriptor Composition;
		Composition.Add<FTestFragment_Int>();
		Composition.Add<FTestConstSharedFragment_Int>();
		const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(Composition);

		// Create entities with DIFFERENT shared values — they should be in the SAME archetype
		// (shared fragment TYPES define archetype composition, VALUES are per-chunk)
		FMassArchetypeSharedFragmentValues SharedValuesA;
		SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(42));
		FMassArchetypeSharedFragmentValues SharedValuesB;
		SharedValuesB.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(99));

		TArray<FMassEntityHandle> EntitiesA;
		EntityManager->BatchCreateEntities(Archetype, SharedValuesA, 5, EntitiesA);
		TArray<FMassEntityHandle> EntitiesB;
		EntityManager->BatchCreateEntities(Archetype, SharedValuesB, 5, EntitiesB);

		// Different shared values → same archetype (archetype is defined by types, not values)
		const FMassArchetypeHandle ArchA = EntityManager->GetArchetypeForEntity(EntitiesA[0]);
		const FMassArchetypeHandle ArchB = EntityManager->GetArchetypeForEntity(EntitiesB[0]);
		AITEST_EQUAL("Different shared values produce SAME archetype", ArchA, ArchB);

		// But queries observe distinct shared values per chunk
		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddConstSharedRequirement<FTestConstSharedFragment_Int>();

		TSet<int32> ObservedSharedValues;
		FMassExecutionContext ExecutionContext(*EntityManager);
		Query.ForEachEntityChunk(ExecutionContext,
			[&ObservedSharedValues](FMassExecutionContext& Context)
		{
			const FTestConstSharedFragment_Int& SharedFrag = Context.GetConstSharedFragment<FTestConstSharedFragment_Int>();
			ObservedSharedValues.Add(SharedFrag.Value);
		});

		AITEST_EQUAL("Query observed 2 distinct shared values across chunks", ObservedSharedValues.Num(), 2);
		AITEST_TRUE("Observed value 42", ObservedSharedValues.Contains(42));
		AITEST_TRUE("Observed value 99", ObservedSharedValues.Contains(99));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetype_SharedFragmentPerChunkStorage, "System.Mass.Coverage.Archetype.SharedFragmentPerChunkStorage");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
