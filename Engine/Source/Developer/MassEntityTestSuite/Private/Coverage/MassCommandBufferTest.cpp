// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassCommandBuffer.h"
#include "MassEntityView.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FCommands_CancelCommands : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);

		// Queue commands
		EntityManager->Defer().AddFragment<FTestFragment_Int>(Entity);
		EntityManager->Defer().AddTag<FTestTag_A>(Entity);

		// Cancel all pending commands
		EntityManager->Defer().CancelCommands();

		// Flush — should be a no-op
		EntityManager->FlushCommands();

		// Entity should still be in FloatsArchetype with no Int fragment and no tag
		AITEST_EQUAL("Entity still in original archetype", EntityManager->GetArchetypeForEntity(Entity), FloatsArchetype);

		FMassEntityView View(*EntityManager, Entity);
		AITEST_NULL("FTestFragment_Int should NOT be present", View.GetFragmentDataPtr<FTestFragment_Int>());
		AITEST_FALSE("FTestTag_A should NOT be present", View.HasTag<FTestTag_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_CancelCommands, "System.Mass.Coverage.CommandBuffer.CancelCommands");

struct FCommands_MoveAppend : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(EmptyArchetype);

		// Use the entity manager's deferred command buffer to build up commands,
		// then test MoveAppend by creating a separate buffer and merging it in.
		// Note: FMassCommandBuffer() has a default constructor only, and Flush() is private
		// (only callable by FMassEntityManager). So we work through EntityManager->Defer().
		FMassCommandBuffer SeparateBuffer;
		SeparateBuffer.AddFragment<FTestFragment_Int>(Entity);

		AITEST_TRUE("Separate buffer has pending commands", SeparateBuffer.HasPendingCommands());

		// Add Float via the manager's deferred buffer
		EntityManager->Defer().AddFragment<FTestFragment_Float>(Entity);

		// MoveAppend the separate buffer into the manager's deferred buffer
		EntityManager->Defer().MoveAppend(SeparateBuffer);

		AITEST_FALSE("Separate buffer is empty after MoveAppend", SeparateBuffer.HasPendingCommands());
		AITEST_TRUE("Deferred buffer has pending commands after MoveAppend", EntityManager->Defer().HasPendingCommands());

		// Flush via the entity manager — both commands should execute
		EntityManager->FlushCommands();

		FMassEntityView View(*EntityManager, Entity);
		AITEST_NOT_NULL("Entity has FTestFragment_Float after flush", View.GetFragmentDataPtr<FTestFragment_Float>());
		AITEST_NOT_NULL("Entity has FTestFragment_Int after flush", View.GetFragmentDataPtr<FTestFragment_Int>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_MoveAppend, "System.Mass.Coverage.CommandBuffer.MoveAppend");

struct FCommands_HasPendingCommands : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		AITEST_FALSE("No pending commands initially", EntityManager->Defer().HasPendingCommands());

		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		EntityManager->Defer().AddTag<FTestTag_A>(Entity);

		AITEST_TRUE("Has pending commands after AddTag", EntityManager->Defer().HasPendingCommands());

		EntityManager->FlushCommands();

		AITEST_FALSE("No pending commands after flush", EntityManager->Defer().HasPendingCommands());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_HasPendingCommands, "System.Mass.Coverage.CommandBuffer.HasPendingCommands");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
