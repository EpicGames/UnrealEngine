// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FComposition_Difference : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		// A = {FragFloat, FragInt, TagA}
		FMassArchetypeCompositionDescriptor A;
		A.Add<FTestFragment_Float>();
		A.Add<FTestFragment_Int>();
		A.Add<FTestTag_A>();

		// B = {FragInt, FragBool, TagA, TagB}
		FMassArchetypeCompositionDescriptor B;
		B.Add<FTestFragment_Int>();
		B.Add<FTestFragment_Bool>();
		B.Add<FTestTag_A>();
		B.Add<FTestTag_B>();

		// A.CalculateDifference(B) = elements in A not in B = {FragFloat}
		const FMassArchetypeCompositionDescriptor AMinusB = A.CalculateDifference(B);
		AITEST_TRUE("A-B contains FragFloat", AMinusB.Contains<FTestFragment_Float>());
		AITEST_FALSE("A-B does not contain FragInt", AMinusB.Contains<FTestFragment_Int>());
		AITEST_FALSE("A-B does not contain TagA", AMinusB.Contains<FTestTag_A>());
		AITEST_FALSE("A-B does not contain FragBool", AMinusB.Contains<FTestFragment_Bool>());
		AITEST_EQUAL("A-B has 1 stored type", AMinusB.CountStoredTypes(), 1);

		// B.CalculateDifference(A) = elements in B not in A = {FragBool, TagB}
		const FMassArchetypeCompositionDescriptor BMinusA = B.CalculateDifference(A);
		AITEST_TRUE("B-A contains FragBool", BMinusA.Contains<FTestFragment_Bool>());
		AITEST_TRUE("B-A contains TagB", BMinusA.Contains<FTestTag_B>());
		AITEST_FALSE("B-A does not contain FragInt", BMinusA.Contains<FTestFragment_Int>());
		AITEST_EQUAL("B-A has 2 stored types", BMinusA.CountStoredTypes(), 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FComposition_Difference, "System.Mass.Coverage.Composition.Difference");

struct FComposition_HasAll : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassArchetypeCompositionDescriptor Full;
		Full.Add<FTestFragment_Float>();
		Full.Add<FTestFragment_Int>();
		Full.Add<FTestTag_A>();

		FMassArchetypeCompositionDescriptor Subset;
		Subset.Add<FTestFragment_Float>();
		Subset.Add<FTestTag_A>();

		FMassArchetypeCompositionDescriptor NonSubset;
		NonSubset.Add<FTestFragment_Float>();
		NonSubset.Add<FTestFragment_Bool>(); // Not in Full

		AITEST_TRUE("Full HasAll of valid subset", Full.HasAll(Subset));
		AITEST_FALSE("Full does NOT HasAll of non-subset", Full.HasAll(NonSubset));

		// Empty descriptor is a subset of everything
		FMassArchetypeCompositionDescriptor Empty;
		AITEST_TRUE("Full HasAll of empty", Full.HasAll(Empty));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FComposition_HasAll, "System.Mass.Coverage.Composition.HasAll");

struct FComposition_CountStoredTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassArchetypeCompositionDescriptor Desc;
		AITEST_EQUAL("Empty composition has 0 types", Desc.CountStoredTypes(), 0);

		Desc.Add<FTestFragment_Float>();
		Desc.Add<FTestFragment_Int>();
		Desc.Add<FTestFragment_Bool>();
		Desc.Add<FTestTag_A>();
		Desc.Add<FTestTag_B>();
		Desc.Add<FTestSharedFragment_Int>();

		AITEST_EQUAL("Composition with 3 fragments + 2 tags + 1 shared = 6 types", Desc.CountStoredTypes(), 6);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FComposition_CountStoredTypes, "System.Mass.Coverage.Composition.CountStoredTypes");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
