// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityView.h"
#include "MassEntityTypes.h"
#include "MassExecutionContext.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

#if WITH_MASSENTITY_DEBUG
struct FManager_EntityCompaction : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
		// Create enough entities to span at least 3 chunks
		const int32 TotalEntities = EntitiesPerChunk * 3;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, TotalEntities, Entities);

		// Set values so we can verify data integrity after compaction
		for (int32 i = 0; i < TotalEntities; ++i)
		{
			EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entities[i]).Value = i;
		}

		// Destroy every other entity to create fragmentation
		TArray<FMassEntityHandle> SurvivorEntities;
		for (int32 i = 0; i < TotalEntities; ++i)
		{
			if (i % 2 == 0)
			{
				EntityManager->DestroyEntity(Entities[i]);
			}
			else
			{
				SurvivorEntities.Add(Entities[i]);
			}
		}

		const int32 SurvivorCount = SurvivorEntities.Num();

		// Compact with unlimited time budget
		EntityManager->DoEntityCompaction(TNumericLimits<double>::Max());

		// Verify all surviving entities have intact data
		for (int32 i = 0; i < SurvivorCount; ++i)
		{
			AITEST_TRUE("Survivor entity is still valid", EntityManager->IsEntityValid(SurvivorEntities[i]));
			const int32 OriginalIndex = i * 2 + 1; // Odd indices survived
			const int32 ActualValue = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(SurvivorEntities[i]).Value;
			AITEST_EQUAL("Data integrity after compaction", ActualValue, OriginalIndex);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FManager_EntityCompaction, "System.Mass.Coverage.Manager.EntityCompaction");
#endif // WITH_MASSENTITY_DEBUG

struct FManager_SwapTagsForEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add TagA
		EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
		{
			FMassEntityView View(*EntityManager, Entity);
			AITEST_TRUE("Entity has TagA before swap", View.HasTag<FTestTag_A>());
			AITEST_FALSE("Entity does not have TagB before swap", View.HasTag<FTestTag_B>());
		}

		// Swap TagA -> TagB
		EntityManager->SwapTagsForEntity(Entity, FTestTag_A::StaticStruct(), FTestTag_B::StaticStruct());

		{
			FMassEntityView View(*EntityManager, Entity);
			AITEST_FALSE("Entity does not have TagA after swap", View.HasTag<FTestTag_A>());
			AITEST_TRUE("Entity has TagB after swap", View.HasTag<FTestTag_B>());
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FManager_SwapTagsForEntity, "System.Mass.Coverage.Manager.SwapTagsForEntity");

struct FManager_ArchetypeTraversal : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create archetypes
		EntityManager->CreateEntity(FloatsArchetype);
		EntityManager->CreateEntity(IntsArchetype);
		EntityManager->CreateEntity(FloatsIntsArchetype);

		// Traverse FloatsIntsArchetype and collect fragment types
		TArray<const UScriptStruct*> CollectedTypes;
		FMassEntityManager::ForEachArchetypeFragmentType(FloatsIntsArchetype, [&CollectedTypes](const UScriptStruct* FragmentType)
		{
			CollectedTypes.Add(FragmentType);
		});

		AITEST_EQUAL("FloatsInts archetype has 2 fragment types", CollectedTypes.Num(), 2);

		const bool bHasFloat = CollectedTypes.Contains(FTestFragment_Float::StaticStruct());
		const bool bHasInt = CollectedTypes.Contains(FTestFragment_Int::StaticStruct());
		AITEST_TRUE("Contains FTestFragment_Float", bHasFloat);
		AITEST_TRUE("Contains FTestFragment_Int", bHasInt);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FManager_ArchetypeTraversal, "System.Mass.Coverage.Manager.ArchetypeTraversal");

struct FManager_BatchAddElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 NumEntities = 100;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

		// Verify all start without Int fragment
		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_EQUAL("Initial archetype is Floats", EntityManager->GetArchetypeForEntity(Entity), FloatsArchetype);
		}

		// Batch add FTestFragment_Int to all entities
		EntityManager->BatchAddElementToEntities(Entities, FTestFragment_Int::StaticStruct());

		// All entities should now be in FloatsInts archetype
		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_EQUAL("Migrated to FloatsInts archetype", EntityManager->GetArchetypeForEntity(Entity), FloatsIntsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FManager_BatchAddElements, "System.Mass.Coverage.Manager.BatchAddElements");

struct FManager_BatchRemoveElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 NumEntities = 100;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumEntities, Entities);

		// Batch remove FTestFragment_Int from all entities
		EntityManager->BatchRemoveElementFromEntities(Entities, FTestFragment_Int::StaticStruct());

		// All entities should now be in Floats-only archetype
		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_EQUAL("Migrated to Floats archetype", EntityManager->GetArchetypeForEntity(Entity), FloatsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FManager_BatchRemoveElements, "System.Mass.Coverage.Manager.BatchRemoveElements");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
