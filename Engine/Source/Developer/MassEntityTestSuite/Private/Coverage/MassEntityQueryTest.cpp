// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"
#include "MassArchetypeTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{
#if WITH_MASSENTITY_DEBUG
struct FQuery_ChunkFilter_Basic : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create archetype with chunk fragment
		const UScriptStruct* Types[] = { FTestFragment_Int::StaticStruct(), FTestChunkFragment_Int::StaticStruct() };
		const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(MakeArrayView(Types));

		// Find how many entities fit per chunk, then create enough to span multiple chunks
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(Archetype);
		const int32 NumChunks = 3;
		const int32 TotalEntities = EntitiesPerChunk * NumChunks;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(Archetype, TotalEntities, Entities);

		// Set up query with chunk filter
		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadWrite);

		// Set chunk fragment values to distinguish chunks — we'll write a value per chunk
		// First iteration: set chunk fragment values
		{
			int32 ChunkIndex = 0;
			FMassExecutionContext SetupContext(*EntityManager);
			Query.ForEachEntityChunk(SetupContext, [&ChunkIndex](FMassExecutionContext& Context)
			{
				FTestChunkFragment_Int& ChunkFrag = Context.GetMutableChunkFragment<FTestChunkFragment_Int>();
				ChunkFrag.Value = ChunkIndex;
				ChunkIndex++;
			});
		}

		// Now set chunk filter: only accept chunks where ChunkFragment value > 0
		const int32 Threshold = 0;
		Query.SetChunkFilter([Threshold](const FMassExecutionContext& Context) -> bool
		{
			const FTestChunkFragment_Int& ChunkFrag = Context.GetChunkFragment<FTestChunkFragment_Int>();
			return ChunkFrag.Value > Threshold;
		});

		AITEST_TRUE("HasChunkFilter returns true after SetChunkFilter", Query.HasChunkFilter());

		// Count entities visited with filter
		int32 FilteredEntityCount = 0;
		{
			FMassExecutionContext FilterContext(*EntityManager);
			Query.ForEachEntityChunk(FilterContext, [&FilteredEntityCount](FMassExecutionContext& Context)
			{
				FilteredEntityCount += Context.GetNumEntities();
			});
		}

		// Chunk 0 has Value=0 (filtered out), chunks 1 and 2 have Value>0 (kept)
		const int32 ExpectedEntities = EntitiesPerChunk * (NumChunks - 1);
		AITEST_EQUAL("Only entities in chunks passing filter were visited", FilteredEntityCount, ExpectedEntities);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQuery_ChunkFilter_Basic, "System.Mass.Coverage.Query.ChunkFilter.Basic");

struct FQuery_ChunkFilter_Clear : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const UScriptStruct* Types[] = { FTestFragment_Int::StaticStruct(), FTestChunkFragment_Int::StaticStruct() };
		const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(MakeArrayView(Types));

		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(Archetype);
		const int32 TotalEntities = EntitiesPerChunk * 2;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(Archetype, TotalEntities, Entities);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadWrite);

		// Set chunk 0 value to 0, chunk 1 value to 1
		{
			int32 ChunkIndex = 0;
			FMassExecutionContext SetupContext(*EntityManager);
			Query.ForEachEntityChunk(SetupContext, [&ChunkIndex](FMassExecutionContext& Context)
			{
				Context.GetMutableChunkFragment<FTestChunkFragment_Int>().Value = ChunkIndex++;
			});
		}

		AITEST_FALSE("No chunk filter initially", Query.HasChunkFilter());

		// Set filter to skip chunk 0
		Query.SetChunkFilter([](const FMassExecutionContext& Context) -> bool
		{
			return Context.GetChunkFragment<FTestChunkFragment_Int>().Value > 0;
		});
		AITEST_TRUE("Has chunk filter after set", Query.HasChunkFilter());

		int32 FilteredCount = 0;
		{
			FMassExecutionContext ExecContext(*EntityManager);
			Query.ForEachEntityChunk(ExecContext, [&FilteredCount](FMassExecutionContext& Context)
			{
				FilteredCount += Context.GetNumEntities();
			});
		}
		AITEST_EQUAL("Filtered count is one chunk", FilteredCount, EntitiesPerChunk);

		// Clear filter
		Query.ClearChunkFilter();
		AITEST_FALSE("No chunk filter after clear", Query.HasChunkFilter());

		int32 UnfilteredCount = 0;
		{
			FMassExecutionContext ExecContext(*EntityManager);
			Query.ForEachEntityChunk(ExecContext, [&UnfilteredCount](FMassExecutionContext& Context)
			{
				UnfilteredCount += Context.GetNumEntities();
			});
		}
		AITEST_EQUAL("Unfiltered count is all entities", UnfilteredCount, TotalEntities);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQuery_ChunkFilter_Clear, "System.Mass.Coverage.Query.ChunkFilter.Clear");

struct FQuery_ChunkFilter_WithCollections : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const UScriptStruct* Types[] = { FTestFragment_Int::StaticStruct(), FTestChunkFragment_Int::StaticStruct() };
		const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(MakeArrayView(Types));

		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(Archetype);
		const int32 TotalEntities = EntitiesPerChunk * 3;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(Archetype, TotalEntities, Entities);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadWrite);

		// Tag chunks with indices
		{
			int32 ChunkIndex = 0;
			FMassExecutionContext SetupContext(*EntityManager);
			Query.ForEachEntityChunk(SetupContext, [&ChunkIndex](FMassExecutionContext& Context)
			{
				Context.GetMutableChunkFragment<FTestChunkFragment_Int>().Value = ChunkIndex++;
			});
		}

		// Create a collection from a subset of entities (first 2 chunks)
		TArray<FMassEntityHandle> SubsetEntities(Entities.GetData(), EntitiesPerChunk * 2);
		FMassArchetypeEntityCollection Collection(Archetype, SubsetEntities, FMassArchetypeEntityCollection::NoDuplicates);

		// Set chunk filter to only accept chunk index > 0
		Query.SetChunkFilter([](const FMassExecutionContext& Context) -> bool
		{
			return Context.GetChunkFragment<FTestChunkFragment_Int>().Value > 0;
		});

		// Execute on collection with chunk filter — should only visit chunk 1 entities
		int32 VisitedCount = 0;
		{
			FMassExecutionContext ExecContext(*EntityManager);
			TArray<FMassArchetypeEntityCollection> Collections;
			Collections.Add(MoveTemp(Collection));
			Query.ForEachEntityChunkInCollections(MakeArrayView(Collections), ExecContext, [&VisitedCount](FMassExecutionContext& Context)
			{
				VisitedCount += Context.GetNumEntities();
			});
		}

		// Collection has chunks 0 and 1. Filter removes chunk 0. Only chunk 1 remains.
		AITEST_EQUAL("Only entities in collection AND passing chunk filter were visited", VisitedCount, EntitiesPerChunk);

		Query.ClearChunkFilter();
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQuery_ChunkFilter_WithCollections, "System.Mass.Coverage.Query.ChunkFilter.WithCollections");
#endif // WITH_MASSENTITY_DEBUG
} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
