// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityView.h"
#include "MassEntityTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FEntityView_TypeErasedFragmentAccess : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
		FMassEntityView View(*EntityManager, Entity);

		// Typed access
		FTestFragment_Int& TypedRef = View.GetFragmentData<FTestFragment_Int>();

		// Type-erased access
		FStructView StructView = View.GetFragmentDataStruct(FTestFragment_Int::StaticStruct());

		AITEST_TRUE("Type-erased view is valid", StructView.IsValid());
		AITEST_EQUAL("Type-erased pointer matches typed pointer",
			StructView.GetMemory(), reinterpret_cast<const uint8*>(&TypedRef));

		// Verify we can read through the type-erased view
		FTestFragment_Int* ErasedPtr = StructView.GetPtr<FTestFragment_Int>();
		AITEST_NOT_NULL("Can cast type-erased view back to typed", ErasedPtr);
		AITEST_EQUAL("Values match", ErasedPtr->Value, TypedRef.Value);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_TypeErasedFragmentAccess, "System.Mass.Coverage.EntityView.TypeErasedFragmentAccess");

struct FEntityView_TypeErasedSharedAccess : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Build archetype with shared and const shared fragments
		FMassArchetypeCompositionDescriptor Composition;
		Composition.Add<FTestFragment_Int>();
		Composition.Add<FTestSharedFragment_Int>();
		Composition.Add<FTestConstSharedFragment_Int>();

		FMassArchetypeSharedFragmentValues SharedValues;
		const FSharedStruct SharedInt = FSharedStruct::Make<FTestSharedFragment_Int>(42);
		SharedValues.Add(SharedInt);
		const FConstSharedStruct ConstSharedInt = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(99);
		SharedValues.Add(ConstSharedInt);
		SharedValues.Sort();

		// CreateArchetype takes only composition — shared values are passed to BatchCreateEntities
		const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(Composition);
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(Archetype, SharedValues, 1, Entities);
		const FMassEntityHandle Entity = Entities[0];
		FMassEntityView View(*EntityManager, Entity);

		// Test type-erased const shared access
		FConstStructView ConstSharedView = View.GetConstSharedFragmentDataStruct(FTestConstSharedFragment_Int::StaticStruct());
		AITEST_TRUE("Const shared type-erased view is valid", ConstSharedView.IsValid());
		const FTestConstSharedFragment_Int* ConstData = ConstSharedView.GetPtr<FTestConstSharedFragment_Int>();
		AITEST_NOT_NULL("Const shared data is accessible", ConstData);
		AITEST_EQUAL("Const shared value matches", ConstData->Value, 99);

		// Test type-erased mutable shared access
		FStructView SharedView = View.GetSharedFragmentDataStruct(FTestSharedFragment_Int::StaticStruct());
		AITEST_TRUE("Shared type-erased view is valid", SharedView.IsValid());
		FTestSharedFragment_Int* SharedData = SharedView.GetPtr<FTestSharedFragment_Int>();
		AITEST_NOT_NULL("Shared data is accessible", SharedData);
		AITEST_EQUAL("Shared value matches", SharedData->Value, 42);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_TypeErasedSharedAccess, "System.Mass.Coverage.EntityView.TypeErasedSharedAccess");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
