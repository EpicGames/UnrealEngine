// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"
#include "MassObserverProcessor.h"
#include "MassExecutor.h"
#include "Engine/Engine.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FExecutionContext_SubsystemAccessChecked : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
		Query.AddSubsystemRequirement<UMassTestWorldSubsystem>(EMassFragmentAccess::ReadWrite);
		Query.AddSubsystemRequirement<UMassTestEngineSubsystem>(EMassFragmentAccess::ReadOnly);

		bool bWorldSubsystemAccessSucceeded = false;
		bool bEngineSubsystemAccessSucceeded = false;

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0);
		Query.ForEachEntityChunk(ExecutionContext, [&bWorldSubsystemAccessSucceeded, &bEngineSubsystemAccessSucceeded](FMassExecutionContext& Context)
		{
			UMassTestWorldSubsystem& WorldSubsystem = Context.GetMutableSubsystemChecked<UMassTestWorldSubsystem>();
			bWorldSubsystemAccessSucceeded = true;

			const UMassTestEngineSubsystem& EngineSubsystem = Context.GetSubsystemChecked<UMassTestEngineSubsystem>();
			bEngineSubsystemAccessSucceeded = true;
		});

		AITEST_TRUE("World subsystem access via GetMutableSubsystemChecked succeeded", bWorldSubsystemAccessSucceeded);
		AITEST_TRUE("Engine subsystem access via GetSubsystemChecked succeeded", bEngineSubsystemAccessSucceeded);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecutionContext_SubsystemAccessChecked, "System.Mass.Coverage.ExecutionContext.SubsystemAccessChecked");

struct FExecutionContext_AuxData : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Set up observer that checks AuxData
		UMassTestProcessorBase* ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		ObserverProcessor->EntityQuery.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);

		bool bAuxDataValid = false;
		EMassObservedOperation ObservedOp = EMassObservedOperation::MAX;
		const UScriptStruct* ObservedType = nullptr;

		ObserverProcessor->ForEachEntityChunkExecutionFunction = [&bAuxDataValid, &ObservedOp, &ObservedType](FMassExecutionContext& Context)
		{
			const FInstancedStruct& AuxData = Context.GetAuxData();
			if (AuxData.IsValid())
			{
				const FMassObserverExecutionContext* ObserverContext = AuxData.GetPtr<FMassObserverExecutionContext>();
				if (ObserverContext)
				{
					bAuxDataValid = true;
					ObservedOp = ObserverContext->GetOperationType();
					ObservedType = ObserverContext->GetCurrentType();
				}
			}
		};

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

		// Trigger observer via deferred tag addition
		EntityManager->Defer().AddTag<FTestTag_A>(Entity);
		EntityManager->FlushCommands();

		AITEST_TRUE("AuxData contained valid FMassObserverExecutionContext", bAuxDataValid);
		AITEST_EQUAL("Observed operation was AddElement", ObservedOp, EMassObservedOperation::AddElement);
		AITEST_EQUAL("Observed type was FTestTag_A", ObservedType, FTestTag_A::StaticStruct());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecutionContext_AuxData, "System.Mass.Coverage.ExecutionContext.AuxData");

struct FExecutionContext_MidExecutionFlush : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entity with Float fragment only
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		bool bFlushCalled = false;

		bool bHasFragmentAfterFlush = false;
		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0);
		Query.ForEachEntityChunk(ExecutionContext, [&bFlushCalled, &bHasFragmentAfterFlush, this](FMassExecutionContext& Context)
		{
			// Defer adding Int fragment to a different entity (not the one being iterated)
			// and flush mid-execution
			const FMassEntityHandle NewEntity = EntityManager->CreateEntity(EmptyArchetype);
			Context.Defer().AddFragment<FTestFragment_Int>(NewEntity);
			Context.FlushDeferred();
			bFlushCalled = true;

			// Check if the deferred command executed - new entity should now have FTestFragment_Int
			bHasFragmentAfterFlush = EntityManager->GetArchetypeForEntity(NewEntity).IsValid()
				&& FMassEntityView(*EntityManager, NewEntity).GetFragmentDataPtr<FTestFragment_Int>() != nullptr;
		});

		AITEST_TRUE("Deferred AddFragment executed after FlushDeferred", bHasFragmentAfterFlush);

		AITEST_TRUE("FlushDeferred was called during iteration", bFlushCalled);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecutionContext_MidExecutionFlush, "System.Mass.Coverage.ExecutionContext.MidExecutionFlush");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
