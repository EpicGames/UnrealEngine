// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "MassObserverProcessor.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FObserver_BatchElementNotification : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entities in an archetype without TagA
		const int32 NumEntities = 50;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Set up observer for FTestFragment_Float addition
		UMassTestProcessorBase* ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		int32 ObservedEntityCount = 0;
		ObserverProcessor->ForEachEntityChunkExecutionFunction = [&ObservedEntityCount](FMassExecutionContext& Context)
		{
			ObservedEntityCount += Context.GetNumEntities();
		};

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

		// Batch add element to all entities
		EntityManager->BatchAddElementToEntities(Entities, FTestFragment_Float::StaticStruct());

		AITEST_EQUAL("Observer processed all entities from batch add", ObservedEntityCount, NumEntities);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FObserver_BatchElementNotification, "System.Mass.Coverage.Observer.BatchElementNotification");

struct FObserver_FlushOrdering : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Track observer firing order
		TArray<FString> FiringOrder;

		// Observer for TagA addition
		UMassTestProcessorBase* AddObserver = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		AddObserver->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		AddObserver->EntityQuery.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		AddObserver->ForEachEntityChunkExecutionFunction = [&FiringOrder](FMassExecutionContext& Context)
		{
			FiringOrder.Add(TEXT("Add"));
		};

		// Observer for TagA removal
		UMassTestProcessorBase* RemoveObserver = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		RemoveObserver->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		RemoveObserver->ForEachEntityChunkExecutionFunction = [&FiringOrder](FMassExecutionContext& Context)
		{
			FiringOrder.Add(TEXT("Remove"));
		};

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add, AddObserver);
		ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Remove, RemoveObserver);

		// Queue add then remove in same flush
		EntityManager->Defer().AddTag<FTestTag_A>(Entity);
		EntityManager->Defer().RemoveTag<FTestTag_A>(Entity);
		EntityManager->FlushCommands();

		// Both observers should have fired in order: add then remove
		AITEST_EQUAL("Both observers fired", FiringOrder.Num(), 2);
		if (FiringOrder.Num() == 2)
		{
			AITEST_EQUAL("Add observer fired first", FiringOrder[0], TEXT("Add"));
			AITEST_EQUAL("Remove observer fired second", FiringOrder[1], TEXT("Remove"));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FObserver_FlushOrdering, "System.Mass.Coverage.Observer.FlushOrdering");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
