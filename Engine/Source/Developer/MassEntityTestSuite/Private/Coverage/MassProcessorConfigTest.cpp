// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingTypes.h"
#include "MassProcessingContext.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FProcessor_DynamicMarking : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		AITEST_FALSE("Processor is not dynamic by default", Processor->IsDynamic());

		Processor->MarkAsDynamic();
		AITEST_TRUE("Processor is dynamic after MarkAsDynamic", Processor->IsDynamic());

		// Verify dynamic processor still executes
		EntityManager->CreateEntity(FloatsArchetype);
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		int32 ExecutionCount = 0;
		Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
		{
			++ExecutionCount;
		};

		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);

		AITEST_EQUAL("Dynamic processor executed successfully", ExecutionCount, 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessor_DynamicMarking, "System.Mass.Coverage.ProcessorConfig.DynamicMarking");

struct FProcessor_QueryBasedPruning : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entities with Float fragment only
		EntityManager->CreateEntity(FloatsArchetype);

		// Create processor that requires Bool fragment (not present in any entity)
		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		Processor->EntityQuery.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadOnly);

		int32 ExecutionCount = 0;
		Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
		{
			++ExecutionCount;
		};

		// Execute — processor should not execute because no entities match
		FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);

		AITEST_EQUAL("Processor with no matching entities did not execute", ExecutionCount, 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessor_QueryBasedPruning, "System.Mass.Coverage.ProcessorConfig.QueryBasedPruning");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
