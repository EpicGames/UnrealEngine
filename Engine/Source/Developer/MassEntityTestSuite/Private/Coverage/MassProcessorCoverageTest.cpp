// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingTypes.h"
#include "MassProcessingContext.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FProcessor_InitialState : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		AITEST_TRUE("Newly created processor is active by default", Processor->IsActive());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessor_InitialState, "System.Mass.Coverage.Processor.InitialState");

struct FProcessor_ActiveInactive : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		EntityManager->CreateEntity(FloatsArchetype);

		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		int32 ExecutionCount = 0;
		Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
		{
			++ExecutionCount;
		};

		// Make inactive and execute
		Processor->MakeInactive();
		AITEST_FALSE("Processor is not active after MakeInactive", Processor->IsActive());

		{
			FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
			UE::Mass::Executor::Run(*Processor, ProcessingContext);
		}
		AITEST_EQUAL("Inactive processor did not execute", ExecutionCount, 0);

		// Make active and execute
		Processor->MakeActive();
		AITEST_TRUE("Processor is active after MakeActive", Processor->IsActive());

		{
			FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
			UE::Mass::Executor::Run(*Processor, ProcessingContext);
		}
		AITEST_EQUAL("Active processor executed once", ExecutionCount, 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessor_ActiveInactive, "System.Mass.Coverage.Processor.ActiveInactive");

struct FProcessor_OneShot : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		EntityManager->CreateEntity(FloatsArchetype);

		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		int32 ExecutionCount = 0;
		Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
		{
			++ExecutionCount;
		};

		Processor->MakeOneShot();
		AITEST_TRUE("OneShot processor is active before first execution", Processor->IsActive());

		// First execution — should run
		{
			FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
			UE::Mass::Executor::Run(*Processor, ProcessingContext);
		}
		AITEST_EQUAL("OneShot processor executed on first run", ExecutionCount, 1);
		AITEST_FALSE("OneShot processor is inactive after first execution", Processor->IsActive());

		// Second execution — should NOT run
		{
			FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
			UE::Mass::Executor::Run(*Processor, ProcessingContext);
		}
		AITEST_EQUAL("OneShot processor did not execute on second run", ExecutionCount, 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessor_OneShot, "System.Mass.Coverage.Processor.OneShot");

struct FProcessor_OneShotReactivation : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		EntityManager->CreateEntity(FloatsArchetype);

		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		int32 ExecutionCount = 0;
		Processor->ForEachEntityChunkExecutionFunction = [&ExecutionCount](FMassExecutionContext& Context)
		{
			++ExecutionCount;
		};

		// OneShot, execute, auto-deactivate
		Processor->MakeOneShot();
		{
			FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
			UE::Mass::Executor::Run(*Processor, ProcessingContext);
		}
		AITEST_EQUAL("Executed once", ExecutionCount, 1);
		AITEST_FALSE("Inactive after oneshot", Processor->IsActive());

		// Reactivate
		Processor->MakeActive();
		AITEST_TRUE("Active after reactivation", Processor->IsActive());

		{
			FMassProcessingContext ProcessingContext(*EntityManager, 0.f);
			UE::Mass::Executor::Run(*Processor, ProcessingContext);
		}
		AITEST_EQUAL("Executed again after reactivation", ExecutionCount, 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessor_OneShotReactivation, "System.Mass.Coverage.Processor.OneShotReactivation");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
