// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityBuilder.h"
#include "MassRelationManager.h"
#include "MassEntityTestTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FCoverageRelationTestBase : FEntityTestBase
{
	UScriptStruct* RelationTypeA = nullptr;
	UScriptStruct* RelationTypeB = nullptr;
	UE::Mass::FTypeHandle RelationHandleA;
	UE::Mass::FTypeHandle RelationHandleB;
	UE::Mass::FRelationManager* RelationManagerPtr = nullptr;

	static TNotNull<UScriptStruct*> MakeRelationType(FName TypeName)
	{
		UScriptStruct* NewStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(),
			FName(*FString::Printf(TEXT("CoverageRelation_%s"), *TypeName.ToString())), RF_Public);
		NewStruct->SetSuperStruct(FMassRelation::StaticStruct());
		return NewStruct;
	}

	static UE::Mass::FRelationTypeTraits CreateRelationTraits(TNotNull<UScriptStruct*> RelationType)
	{
		UE::Mass::FRelationTypeTraits Traits(RelationType);
		Traits.RoleTraits[static_cast<uint8>(UE::Mass::ERelationRole::Object)].DestructionPolicy = UE::Mass::ERemovalPolicy::CleanUp;
		Traits.RoleTraits[static_cast<uint8>(UE::Mass::ERelationRole::Subject)].DestructionPolicy = UE::Mass::ERemovalPolicy::CleanUp;
		return Traits;
	}

	virtual bool SetUp() override
	{
		if (FEntityTestBase::SetUp())
		{
			RelationTypeA = MakeRelationType(TEXT("TypeA"));
			RelationHandleA = EntityManager->GetTypeManager().RegisterType(CreateRelationTraits(RelationTypeA));

			RelationTypeB = MakeRelationType(TEXT("TypeB"));
			RelationHandleB = EntityManager->GetTypeManager().RegisterType(CreateRelationTraits(RelationTypeB));

			RelationManagerPtr = &EntityManager->GetRelationManager();
		}
		return RelationHandleA.IsValid() && RelationHandleB.IsValid();
	}
};

struct FRelation_RoleEntities : FCoverageRelationTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(IntsArchetype);
		const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(FloatsArchetype);

		const FMassEntityHandle RelationEntity = RelationManagerPtr->CreateRelationInstance(RelationHandleA, SubjectEntity, ObjectEntity);
		EntityManager->FlushCommands();

		AITEST_TRUE("Relation entity created", RelationEntity.IsValid());

		// Query subjects of the object
		TArray<FMassEntityHandle> Subjects = RelationManagerPtr->GetRelationSubjects(RelationHandleA, ObjectEntity);
		AITEST_EQUAL("One subject found for ObjectEntity", Subjects.Num(), 1);
		if (Subjects.Num() > 0)
		{
			AITEST_EQUAL("Subject is correct entity", Subjects[0], SubjectEntity);
		}

		// Query objects of the subject
		TArray<FMassEntityHandle> Objects = RelationManagerPtr->GetRelationObjects(RelationHandleA, SubjectEntity);
		AITEST_EQUAL("One object found for SubjectEntity", Objects.Num(), 1);
		if (Objects.Num() > 0)
		{
			AITEST_EQUAL("Object is correct entity", Objects[0], ObjectEntity);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRelation_RoleEntities, "System.Mass.Coverage.Relation.RoleEntities");

struct FRelation_MultiType : FCoverageRelationTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle EntityA = EntityManager->CreateEntity(IntsArchetype);
		const FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype);
		const FMassEntityHandle EntityC = EntityManager->CreateEntity(IntsArchetype);

		// Create two different relation types from EntityA
		RelationManagerPtr->CreateRelationInstance(RelationHandleA, EntityA, EntityB);
		RelationManagerPtr->CreateRelationInstance(RelationHandleB, EntityA, EntityC);
		EntityManager->FlushCommands();

		// Query each type independently
		TArray<FMassEntityHandle> ObjectsA = RelationManagerPtr->GetRelationObjects(RelationHandleA, EntityA);
		AITEST_EQUAL("One TypeA relation object", ObjectsA.Num(), 1);
		if (ObjectsA.Num() > 0)
		{
			AITEST_EQUAL("TypeA object is EntityB", ObjectsA[0], EntityB);
		}

		TArray<FMassEntityHandle> ObjectsB = RelationManagerPtr->GetRelationObjects(RelationHandleB, EntityA);
		AITEST_EQUAL("One TypeB relation object", ObjectsB.Num(), 1);
		if (ObjectsB.Num() > 0)
		{
			AITEST_EQUAL("TypeB object is EntityC", ObjectsB[0], EntityC);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRelation_MultiType, "System.Mass.Coverage.Relation.MultiType");

struct FRelation_DataAccess : FCoverageRelationTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(IntsArchetype);
		const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(FloatsArchetype);

		RelationManagerPtr->CreateRelationInstance(RelationHandleA, SubjectEntity, ObjectEntity);
		EntityManager->FlushCommands();

		const bool bIsSubject = RelationManagerPtr->IsSubjectOfRelation(RelationHandleA, SubjectEntity, ObjectEntity);
		AITEST_TRUE("SubjectEntity is subject of relation to ObjectEntity", bIsSubject);

		const bool bIsNotSubject = RelationManagerPtr->IsSubjectOfRelation(RelationHandleA, ObjectEntity, SubjectEntity);
		AITEST_FALSE("ObjectEntity is NOT subject of relation to SubjectEntity", bIsNotSubject);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRelation_DataAccess, "System.Mass.Coverage.Relation.DataAccess");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
