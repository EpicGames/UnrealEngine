// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "Mass/EntityHandle.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FSerialization_EntityHandleRoundTrip : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassEntityHandle Original = EntityManager->CreateEntity(IntsArchetype);
		AITEST_TRUE("Original handle is valid", Original.IsSet());

		// Round-trip through AsNumber/FromNumber
		const uint64 Number = Original.AsNumber();
		const FMassEntityHandle Reconstructed = FMassEntityHandle::FromNumber(Number);

		AITEST_EQUAL("Reconstructed handle equals original", Reconstructed, Original);
		AITEST_TRUE("Reconstructed handle is valid", EntityManager->IsEntityValid(Reconstructed));

		// Verify they reference the same entity data
		FTestFragment_Int& OriginalData = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Original);
		OriginalData.Value = 777;
		const FTestFragment_Int& ReconstructedData = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Reconstructed);
		AITEST_EQUAL("Both handles reference the same entity data", ReconstructedData.Value, 777);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSerialization_EntityHandleRoundTrip, "System.Mass.Coverage.Serialization.EntityHandleRoundTrip");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
