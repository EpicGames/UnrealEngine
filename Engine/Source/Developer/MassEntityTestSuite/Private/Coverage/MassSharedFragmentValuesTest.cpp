// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Coverage
{

struct FSharedValues_Replace : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassArchetypeSharedFragmentValues SharedValues;
		const FSharedStruct OriginalShared = FSharedStruct::Make<FTestSharedFragment_Int>(10);
		SharedValues.Add(OriginalShared);

		// Verify original value
		const FTestSharedFragment_Int* Data = SharedValues.GetSharedFragmentStruct(FTestSharedFragment_Int::StaticStruct()).GetPtr<FTestSharedFragment_Int>();
		AITEST_NOT_NULL("Original shared data accessible", Data);
		AITEST_EQUAL("Original value is 10", Data->Value, 10);

		// Replace with new value
		const FSharedStruct ReplacementShared = FSharedStruct::Make<FTestSharedFragment_Int>(42);
		SharedValues.ReplaceSharedFragments(MakeArrayView(&ReplacementShared, 1));

		// Verify replaced value
		const FTestSharedFragment_Int* ReplacedData = SharedValues.GetSharedFragmentStruct(FTestSharedFragment_Int::StaticStruct()).GetPtr<FTestSharedFragment_Int>();
		AITEST_NOT_NULL("Replaced shared data accessible", ReplacedData);
		AITEST_EQUAL("Replaced value is 42", ReplacedData->Value, 42);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedValues_Replace, "System.Mass.Coverage.SharedFragmentValues.Replace");

struct FSharedValues_DoesMatchComposition : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassArchetypeSharedFragmentValues SharedValues;
		const FSharedStruct SharedInt = FSharedStruct::Make<FTestSharedFragment_Int>(1);
		SharedValues.Add(SharedInt);
		const FConstSharedStruct ConstSharedInt = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(2);
		SharedValues.Add(ConstSharedInt);

		// Matching composition
		FMassArchetypeCompositionDescriptor MatchingDesc;
		MatchingDesc.Add<FTestSharedFragment_Int>();
		MatchingDesc.Add<FTestConstSharedFragment_Int>();

		AITEST_TRUE("Matches composition with same shared types", SharedValues.DoesMatchComposition(MatchingDesc));

		// Non-matching: missing a type
		FMassArchetypeCompositionDescriptor PartialDesc;
		PartialDesc.Add<FTestSharedFragment_Int>();
		// Missing FTestConstSharedFragment_Int

		AITEST_FALSE("Does not match composition with missing type", SharedValues.DoesMatchComposition(PartialDesc));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedValues_DoesMatchComposition, "System.Mass.Coverage.SharedFragmentValues.DoesMatchComposition");

struct FSharedValues_CreateCombined : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		// Create original shared values with one shared fragment
		FMassArchetypeSharedFragmentValues Original;
		const FConstSharedStruct ConstSharedInt = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(10);
		Original.Add(ConstSharedInt);

		// Modification adds a new shared fragment
		FMassArchetypeSharedFragmentValues Modification;
		const FConstSharedStruct ConstSharedFloat = FConstSharedStruct::Make<FTestConstSharedFragment_Float>(3.14f);
		Modification.Add(ConstSharedFloat);

		// New composition includes both original and new types
		FMassElementBitSet NewComposition;
		NewComposition.Add(FTestConstSharedFragment_Int::StaticStruct());
		NewComposition.Add(FTestConstSharedFragment_Float::StaticStruct());

		FMassArchetypeSharedFragmentValues Combined = FMassArchetypeSharedFragmentValues::CreateCombined(
			Original, NewComposition, &Modification);

		// Verify combined has both fragments
		const FTestConstSharedFragment_Int* IntData = Combined.GetConstSharedFragmentStruct(
			FTestConstSharedFragment_Int::StaticStruct()).GetPtr<FTestConstSharedFragment_Int>();
		AITEST_NOT_NULL("Combined has original const shared int", IntData);
		AITEST_EQUAL("Original value preserved", IntData->Value, 10);

		const FTestConstSharedFragment_Float* FloatData = Combined.GetConstSharedFragmentStruct(
			FTestConstSharedFragment_Float::StaticStruct()).GetPtr<FTestConstSharedFragment_Float>();
		AITEST_NOT_NULL("Combined has added const shared float", FloatData);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedValues_CreateCombined, "System.Mass.Coverage.SharedFragmentValues.CreateCombined");

} // UE::Mass::Test::Coverage

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
