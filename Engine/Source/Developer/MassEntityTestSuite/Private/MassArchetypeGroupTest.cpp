// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityTestTypes.h"
#include "Mass/ArchetypeGroup.h"
#include "MassExecutionContext.h"
#include "Algo/Compare.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test
{

using namespace UE::Mass;

auto DescendingGroupSorter = [](const FArchetypeGroupID A, const FArchetypeGroupID B)
	{
		return A > B;
	};

struct FArchetypeGroup_SingleLevelQuery : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 NumEntities = 100;

		FArchetypeGroupType GroupType = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroup"));

		TArray<FMassEntityHandle> Entities;
		TArray<FMassEntityHandle> VerifyEntities;

		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// verify assumptions regarding order of processing
		FMassEntityQuery EntityQuery(EntityManager);
		EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
		EntityQuery.ForEachEntityChunk(ExecutionContext, [&VerifyEntities](const FMassExecutionContext& Context)
		{
			VerifyEntities.Append(Context.GetEntities());
		});

		AITEST_TRUE("Assumptions re order of entity processing is correct", Algo::Compare(Entities, VerifyEntities));

		// we're assigning each entity an individual group.
		for (int32 EntityIndex = 0; EntityIndex < Entities.Num(); ++EntityIndex)
		{
			EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType, EntityIndex), MakeArrayView(&Entities[EntityIndex], 1));
		}

		// by default the groups will be sorted in the ascending order, so the order of entities being processed should not change
		EntityQuery.GroupBy(GroupType);

		VerifyEntities.Reset();
		EntityQuery.ForEachEntityChunk(ExecutionContext, [&VerifyEntities](const FMassExecutionContext& Context)
		{
			VerifyEntities.Append(Context.GetEntities());
		});
		AITEST_TRUE("After grouping the order is expected to remain the same", Algo::Compare(Entities, VerifyEntities));

		EntityQuery.ResetGrouping();
		// this grouping should result in the reversed order of entity processing.
		EntityQuery.GroupBy(GroupType, DescendingGroupSorter);

		VerifyEntities.Reset();
		EntityQuery.ForEachEntityChunk(ExecutionContext, [&VerifyEntities](const FMassExecutionContext& Context)
		{
			// note that we're inserting at index 0 to end up with the ascending order of entities in the VerifyEntities array for ease of comparison
			VerifyEntities.Insert(Context.GetEntities()[0], 0);
		});
		AITEST_TRUE("After descending grouping the order matches expectations", Algo::Compare(Entities, VerifyEntities));
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_SingleLevelQuery, "System.Mass.Archetype.Group.SingleLevelQuery");

struct FArchetypeGroup_MultiLevelQueryBase : FEntityTestBase
{
	TArray<FArchetypeGroupType> GroupTypes;
	TArray<FMassEntityHandle> Entities;
	FMassEntityQuery EntityQuery;

	static constexpr int32 NumEntities = 16;
	TArray<const int32> GroupSizes;

	using FTestFunction = TFunction<bool(FMassExecutionContext&)>;
	TArray<FTestFunction> Tests;

	virtual bool SetUp() override
	{
		if (FEntityTestBase::SetUp())
		{
			EntityQuery.Initialize(EntityManager.ToSharedRef());
			EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);

			Tests.Add([this](FMassExecutionContext& ExecutionContext)
			{
				TArray<TArray<const int32>> ExpectedEntityIndices = {{ 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15 }};
				return TestDefaultOrder(ExecutionContext, ExpectedEntityIndices);
			});

			return true;
		}
		return false;
	}

	virtual bool InstantTest() override
	{
		const int32 GroupsNum = GroupSizes.Num();

		for (int32 GroupIndex = 0; GroupIndex < GroupsNum; ++GroupIndex)
		{
			const FName GroupTypeName(*FString::Printf(TEXT("TestGroup_%d"), GroupIndex));
			GroupTypes.Add(EntityManager->FindOrAddArchetypeGroupType(GroupTypeName));
		}
		
		AITEST_NOT_EQUAL(TEXT("Two differently named group types are not expected to be equal"), GroupTypes[0], GroupTypes[1]);

		TArray<FMassEntityHandle> VerifyEntities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		AssignGroups();

		FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
		
		int32 TestIndex = 0;
		bool bResult = true;
		while(bResult && TestIndex < Tests.Num())
		{
			bResult = Tests[TestIndex++](ExecutionContext);
		} 

		// @TODO: test sorting when some entities are missing some group types - specifically the first one being used for sorting

		return bResult;
	}

	virtual void AssignGroups()
	{
		auto GroupAssigner = [this](auto&& Self, const int32 GroupIndex, TArrayView<FMassEntityHandle> EntitiesView) -> void
			{
				const int32 GroupsNum = GroupSizes.Num();
				if (GroupIndex < GroupsNum)
				{
					const FArchetypeGroupType& GroupType = GroupTypes[GroupIndex];
					const int32 GroupSize = GroupSizes[GroupIndex];
					int32 SubGroupIndex = 0;
					for (int32 EntityIndex = 0; EntityIndex < EntitiesView.Num(); EntityIndex += GroupSize, ++SubGroupIndex)
					{
						const int32 ThisGroupSize = FMath::Min(EntitiesView.Num() - EntityIndex, GroupSize);
						TArrayView<FMassEntityHandle> SubEntities = MakeArrayView(&EntitiesView[EntityIndex], ThisGroupSize);
						EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType, SubGroupIndex), SubEntities);
						Self(Self, GroupIndex + 1, SubEntities);
					}
				}
			};

		GroupAssigner(GroupAssigner, 0, Entities);
	}

	bool TestDefaultOrder(FMassExecutionContext& ExecutionContext, TArray<TArray<const int32>>& ExpectedEntityIndices)
	{
		EntityQuery.ResetGrouping();
		for (const FArchetypeGroupType& GroupType : GroupTypes)
		{
			EntityQuery.GroupBy(GroupType);
		}

		return TestRaw(ExecutionContext, ExpectedEntityIndices);
	}

	bool Test1(FMassExecutionContext& ExecutionContext, TArray<TArray<const int32>>& ExpectedEntityIndices)
	{
		// now we're going to reverse the sorting of group TestGroup_1
		EntityQuery.ResetGrouping();
		EntityQuery.GroupBy(GroupTypes[0]);
		EntityQuery.GroupBy(GroupTypes[1], DescendingGroupSorter);
		EntityQuery.GroupBy(GroupTypes[2]);

		return TestRaw(ExecutionContext, ExpectedEntityIndices);
	}

	bool Test2(FMassExecutionContext& ExecutionContext, TArray<TArray<const int32>>& ExpectedEntityIndices)
	{
		// now we're going to reverse the sorting of the last group, TestGroup_2
		EntityQuery.ResetGrouping();
		EntityQuery.GroupBy(GroupTypes[0]);
		EntityQuery.GroupBy(GroupTypes[1]);
		EntityQuery.GroupBy(GroupTypes[2], DescendingGroupSorter);

		return TestRaw(ExecutionContext, ExpectedEntityIndices);
	}

	bool Test3(FMassExecutionContext& ExecutionContext, TArray<TArray<const int32>>& ExpectedEntityIndices)
	{
		// now we're going to reverse the sorting of the TestGroup_0 and TestGroup_1, and put the latter first
		EntityQuery.ResetGrouping();
		EntityQuery.GroupBy(GroupTypes[1], DescendingGroupSorter);
		EntityQuery.GroupBy(GroupTypes[0], DescendingGroupSorter);
		EntityQuery.GroupBy(GroupTypes[2]);

		return TestRaw(ExecutionContext, ExpectedEntityIndices);
	}

	/** Grouping expected to be set before calling this one */
	bool TestRaw(FMassExecutionContext& ExecutionContext, TArray<TArray<const int32>>& ExpectedEntityIndices)
	{
		TArray<FMassEntityHandle> VerifyEntities;

		EntityQuery.ForEachEntityChunk(ExecutionContext, [&VerifyEntities](const FMassExecutionContext& Context)
		{
			VerifyEntities.Append(Context.GetEntities());
		});

		if (ExpectedEntityIndices.Num() == 1)
		{
			// we expect the strict order
			for (int32 ResultIndex = 0; ResultIndex < VerifyEntities.Num(); ++ResultIndex)
			{
				AITEST_EQUAL("Expected results vs received", VerifyEntities[ResultIndex], Entities[ExpectedEntityIndices[0][ResultIndex]]);
			}
		}
		else
		{
			TArray<FMassEntityHandle> EntitySet;
			int32 ResultIndex = 0;
			for (int32 SetIndex = 0; SetIndex < ExpectedEntityIndices.Num(); ++SetIndex)
			{
				for (const int32 EntityIndex : ExpectedEntityIndices[SetIndex])
				{
					EntitySet.Add(Entities[EntityIndex]);
				}

				const int32 SetSize = ExpectedEntityIndices[SetIndex].Num();
				for (int32 SetIterator = 0; SetIterator < SetSize; ++SetIterator, ++ResultIndex)
				{
					AITEST_TRUE("Result in expected set", EntitySet.Find(VerifyEntities[ResultIndex]) != INDEX_NONE);
				}
				EntitySet.Reset();
			}
		}

		return true;
	}
};

struct FArchetypeGroup_MultiLevelQuery_A : FArchetypeGroup_MultiLevelQueryBase
{
	virtual bool SetUp() override
	{
		GroupSizes = { 8, 4, 2 };

		// This will result in the following:
		// Entity	| TestGroup_0 	| TestGroup_1 	| TestGroup_2
		//  0		|		0		|		0		|		0
		//  1		|		0		|		0		|		0
		//  2		|		0		|		0		|		1
		//  3		|		0		|		0		|		1
		//  4		|		0		|		1		|		0
		//  5		|		0		|		1		|		0
		//  6		|		0		|		1		|		1
		//  7		|		0		|		1		|		1
		//  8		|		1		|		0		|		0
		//  9		|		1		|		0		|		0
		// 10		|		1		|		0		|		1
		// 11		|		1		|		0		|		1
		// 12		|		1		|		1		|		0
		// 13		|		1		|		1		|		0
		// 14		|		1		|		1		|		1
		// 15		|		1		|		1		|		1

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			TArray<TArray<const int32>> ExpectedEntityIndices = {{ 4, 5, 6, 7, 0, 1, 2, 3, 12, 13, 14, 15, 8, 9, 10, 11 }};
			return Test1(ExecutionContext, ExpectedEntityIndices);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			TArray<TArray<const int32>> ExpectedEntityIndices = {{ 2, 3, 0, 1, 6, 7, 4, 5, 10, 11, 8, 9, 14, 15, 12, 13 }};
			return Test2(ExecutionContext, ExpectedEntityIndices);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			TArray<TArray<const int32>> ExpectedEntityIndices = {{ 12, 13, 14, 15, 4, 5, 6, 7, 8, 9, 10, 11, 0, 1, 2, 3 }};
			return Test3(ExecutionContext, ExpectedEntityIndices);
		});

		return FArchetypeGroup_MultiLevelQueryBase::SetUp();
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_MultiLevelQuery_A, "System.Mass.Archetype.Group.MultiLevelQuery.A");

struct FArchetypeGroup_MultiLevelQuery_B : FArchetypeGroup_MultiLevelQueryBase
{
	virtual bool SetUp() override
	{
		GroupSizes = { 5, 3, 1 };

		// This will result in the following:
		// Entity	| TestGroup_0 	| TestGroup_1 	| TestGroup_2
		//  0		|		0		|		0		|		0
		//  1		|		0		|		0		|		1
		//  2		|		0		|		0		|		2
		//  3		|		0		|		1		|		0
		//  4		|		0		|		1		|		1
		//  5		|		1		|		0		|		0
		//  6		|		1		|		0		|		1
		//  7		|		1		|		0		|		2
		//  8		|		1		|		1		|		0
		//  9		|		1		|		1		|		1
		// 10		|		2		|		0		|		0
		// 11		|		2		|		0		|		1
		// 12		|		2		|		0		|		2
		// 13		|		2		|		1		|		0
		// 14		|		2		|		1		|		1
		// 15		|		3		|		0		|		0

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			TArray<TArray<const int32>> ExpectedEntityIndices = {{ 3, 4, 0, 1, 2, 8, 9, 5, 6, 7, 13, 14, 10, 11, 12, 15 }};
			return Test1(ExecutionContext, ExpectedEntityIndices);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			TArray<TArray<const int32>> ExpectedEntityIndices = {{ 2, 1, 0, 4, 3, 7, 6, 5, 9, 8, 12, 11, 10, 14, 13, 15 }};
			return Test2(ExecutionContext, ExpectedEntityIndices);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			TArray<TArray<const int32>> ExpectedEntityIndices = {{ 13, 14, 8, 9, 3, 4, 15, 10, 11, 12, 5, 6, 7, 0, 1, 2 }};
			return Test3(ExecutionContext, ExpectedEntityIndices);
		});

		return FArchetypeGroup_MultiLevelQueryBase::SetUp();
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_MultiLevelQuery_B, "System.Mass.Archetype.Group.MultiLevelQuery.B");

struct FArchetypeGroup_MultiLevelQuery_Complex : FArchetypeGroup_MultiLevelQueryBase
{
	TArray<TArray<const int32>> OderedSets;

	virtual void AssignGroups() override
	{
		const int32 GroupsNum = GroupSizes.Num();
		for (int32 GroupIndex = 0; GroupIndex < GroupsNum; ++GroupIndex)
		{
			const int32 GroupSize = GroupSizes[GroupIndex];
			int32 SubGroupIndex = 0;
			for (int32 EntityIndex = 0; EntityIndex < Entities.Num(); EntityIndex += GroupSize)
			{
				const int32 SubgroupSize = FMath::Min(Entities.Num() - EntityIndex, GroupSize);

				TArrayView<FMassEntityHandle> SubEntities = MakeArrayView(&Entities[EntityIndex], SubgroupSize);
				EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupTypes[GroupIndex], SubGroupIndex++), SubEntities);
			}
		}
	}

	virtual bool SetUp() override
	{
		if (FArchetypeGroup_MultiLevelQueryBase::SetUp() == false)
		{
			return false;
		}

		GroupSizes = { 3, 4, 6 };

		// This will result in the following:
		// Entity	| TestGroup_0 	| TestGroup_1 	| TestGroup_2
		//  0		|		0		|		0		|		0
		//  1		|		0		|		0		|		0
		//  2		|		0		|		0		|		0
		//  3		|		1		|		0		|		0
		//  4		|		1		|		1		|		0
		//  5		|		1		|		1		|		0
		//  6		|		2		|		1		|		1
		//  7		|		2		|		1		|		1
		//  8		|		2		|		2		|		1
		//  9		|		3		|		2		|		1
		// 10		|		3		|		2		|		1
		// 11		|		3		|		2		|		1
		// 12		|		4		|		3		|		2
		// 13		|		4		|		3		|		2
		// 14		|		4		|		3		|		2
		// 15		|		5		|		3		|		2
		OderedSets = {
			/*[0]*/  { 0, 1, 2}
			/*[1]*/, { 3 }
			/*[2]*/, { 4, 5 }
			/*[3]*/, { 6, 7 }
			/*[4]*/, { 8 }
			/*[5]*/, { 9, 10, 11 }
			/*[6]*/, { 12, 13, 14 }
			/*[7]*/, { 15 }
		};

		// dropping the "default order" test since it can produce a slightly different results than expected
		// due to some entities being in the very same group combinations, like {9, 10, 11} and {12, 13, 14}
		Tests.Reset();
		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			EntityQuery.ResetGrouping();
			for (const FArchetypeGroupType& GroupType : GroupTypes)
			{
				EntityQuery.GroupBy(GroupType);
			}

			return TestRaw(ExecutionContext, OderedSets);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			//TArray<TArray<const int32>> ExpectedEntityIndices = {{ 0, 1, 2, 4, 5, 3, 8, 6, 7, 9, 10, 11, 12, 13, 14, 15 }};
			TArray<TArray<const int32>> ExpectedSets = {
				OderedSets[0]
				, OderedSets[2]
				, OderedSets[1]
				, OderedSets[4]
				, OderedSets[3]
				, OderedSets[5]
				, OderedSets[6]
				, OderedSets[7]
			};
			return Test1(ExecutionContext, ExpectedSets);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			return Test2(ExecutionContext, OderedSets);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			// here's what we expect to see with the TestGroup_1 group sorted in descending order
			//TArray<TArray<const int32>> ExpectedEntityIndices = {{ 15, 12, 13, 14, 9, 10, 11, 8, 6, 7, 4, 5, 3, 0, 1, 2 }};
			TArray<TArray<const int32>> ExpectedSets = {
				OderedSets[7]
				, OderedSets[6]
				, OderedSets[5]
				, OderedSets[4]
				, OderedSets[3]
				, OderedSets[2]
				, OderedSets[1]
				, OderedSets[0]
			};
			return Test3(ExecutionContext, ExpectedSets);
		});

		Tests.Add([this](FMassExecutionContext& ExecutionContext)
		{
			EntityQuery.ResetGrouping();
			EntityQuery.GroupBy(GroupTypes[2], DescendingGroupSorter);
			EntityQuery.GroupBy(GroupTypes[0]);
			EntityQuery.GroupBy(GroupTypes[1], DescendingGroupSorter);

			//TArray<TArray<const int32>> ExpectedEntityIndices = {{ 12, 13, 14, 15, 8, 6, 7, 9, 10, 11, 0, 1, 2, 4, 5, 3 }};
			TArray<TArray<const int32>> ExpectedSets = {
				OderedSets[6]
				, OderedSets[7]
				, OderedSets[4]
				, OderedSets[3]
				, OderedSets[5]
				, OderedSets[0]
				, OderedSets[2]
				, OderedSets[1]
			};

			return TestRaw(ExecutionContext, ExpectedSets);
		});

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_MultiLevelQuery_Complex, "System.Mass.Archetype.Group.MultiLevelQuery.Complex");

/** This test ensures the order in which a given entity is grouped in doesn't matter. */
struct FArchetypeGroup_ApplicationOrder : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		uint32 ArbitraryNumber = 1677;

		FArchetypeGroupType GroupType1 = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroup1"));
		FArchetypeGroupType GroupType2 = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroup2"));

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 2, Entities);

		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType1, ArbitraryNumber), MakeArrayView(&Entities[0], 1));
		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType2, ArbitraryNumber), MakeArrayView(&Entities[0], 1));

		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType2, ArbitraryNumber), MakeArrayView(&Entities[1], 1));
		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType1, ArbitraryNumber), MakeArrayView(&Entities[1], 1));

		FMassArchetypeHandle Archetype0 = EntityManager->GetArchetypeForEntity(Entities[0]);
		FMassArchetypeHandle Archetype1 = EntityManager->GetArchetypeForEntity(Entities[1]);

		AITEST_EQUAL("Final archetype target is independent of order of entity grouping", Archetype0, Archetype1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_ApplicationOrder, "System.Mass.Archetype.Group.ApplicationOrder");

struct FArchetypeGroup_Equivalence : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		uint32 ArbitraryNumber = 1677;

		FArchetypeGroupType GroupType1 = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroup1"));
		FArchetypeGroupType GroupType2 = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroup2"));

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 2, Entities);

		// the first entity only gets the first group
		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType1, ArbitraryNumber), MakeArrayView(&Entities[0], 1));
		
		// the second one gets two and then gets removed from the latter group
		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType1, ArbitraryNumber), MakeArrayView(&Entities[1], 1));
		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType2, ArbitraryNumber), MakeArrayView(&Entities[1], 1));
		EntityManager->RemoveEntityFromGroupType(Entities[1], GroupType2);

		FMassArchetypeHandle Archetype0 = EntityManager->GetArchetypeForEntity(Entities[0]);
		FMassArchetypeHandle Archetype1 = EntityManager->GetArchetypeForEntity(Entities[1]);

		AITEST_EQUAL("The archetypes with groups created directly and with groups modifications", Archetype0, Archetype1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_Equivalence, "System.Mass.Archetype.Group.Equivalence");

/** Ensuring that grouping an entity moves it to a different archetype*/
struct FArchetypeGroup_NewArchetype : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		uint32 ArbitraryNumber = 1677;

		FArchetypeGroupType GroupType = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroup"));

		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType, ArbitraryNumber), MakeArrayView(&Entity, 1));
		FMassArchetypeHandle ArchetypeGroup0 = EntityManager->GetArchetypeForEntity(Entity);
		AITEST_NOT_EQUAL("Adding an entity to a group of any type makes it change archetypes", ArchetypeGroup0, IntsArchetype);

		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType, ArbitraryNumber + 1), MakeArrayView(&Entity, 1));
		FMassArchetypeHandle ArchetypeGroup1 = EntityManager->GetArchetypeForEntity(Entity);
		AITEST_NOT_EQUAL("Switching an entity to a different instance of the given group type makes it change archetypes", ArchetypeGroup0, ArchetypeGroup1);

		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType, ArbitraryNumber), MakeArrayView(&Entity, 1));
		FMassArchetypeHandle ArchetypeGroup0B = EntityManager->GetArchetypeForEntity(Entity);
		AITEST_EQUAL("Switching an entity to back to the original group instance makes it change archetypes back to the previous one", ArchetypeGroup0, ArchetypeGroup0B);

		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupType, ArbitraryNumber), MakeArrayView(&Entity, 1));
		FMassArchetypeHandle ArchetypeGroup0C = EntityManager->GetArchetypeForEntity(Entity);
		AITEST_EQUAL("Attempting to add an entity to a group it's already in results in noop", ArchetypeGroup0, ArchetypeGroup0C);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_NewArchetype, "System.Mass.Archetype.Group.NewArchetype");

struct FArchetypeGroup_NonGroupChange : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		uint32 ArbitraryNumber = 1677;

		FArchetypeGroupType GroupType = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroup"));
		const FArchetypeGroupHandle TargetGroupHandle(GroupType, ArbitraryNumber);

		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
		
		EntityManager->BatchGroupEntities(TargetGroupHandle, MakeArrayView(&Entity, 1));
		FMassArchetypeHandle ArchetypeGroup = EntityManager->GetArchetypeForEntity(Entity);

		EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());

		FMassArchetypeHandle ArchetypeTagGroup = EntityManager->GetArchetypeForEntity(Entity);
		AITEST_NOT_EQUAL("The archetype before and after the move", ArchetypeGroup, ArchetypeTagGroup);

		const FArchetypeGroupHandle FinalGroupHandle = EntityManager->GetGroupForEntity(Entity, GroupType);
		AITEST_EQUAL("Entity's group handle before and after tagging", TargetGroupHandle, FinalGroupHandle);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_NonGroupChange, "System.Mass.Archetype.Group.AddTag");

/** Ensuring that grouping an entity moves it to a different archetype*/
struct FArchetypeGroup_Remove : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr uint32 ArbitraryNumber = 1677;
		const FArchetypeGroupType GroupTypeA = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroupA"));
		const FArchetypeGroupType GroupTypeB = EntityManager->FindOrAddArchetypeGroupType(TEXT("TestGroupB"));
		const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(OriginalArchetype, 2, Entities);

		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupTypeA, ArbitraryNumber), Entities);
		const FMassArchetypeHandle ArchetypeGroupA = EntityManager->GetArchetypeForEntity(Entities[0]);
		EntityManager->BatchGroupEntities(FArchetypeGroupHandle(GroupTypeB, ArbitraryNumber), Entities);
		const FMassArchetypeHandle ArchetypeGroupAB = EntityManager->GetArchetypeForEntity(Entities[0]);

		{
			EntityManager->RemoveEntityFromGroupType(Entities[0], GroupTypeA);
			const FMassArchetypeHandle ArchetypeGroupNoA = EntityManager->GetArchetypeForEntity(Entities[0]);

			const FArchetypeGroups& GroupsNoA = EntityManager->GetGroupsForArchetype(ArchetypeGroupNoA);
			AITEST_TRUE("After removing group A from AB entity it's still in group B", GroupsNoA.ContainsType(GroupTypeB));
			AITEST_FALSE("After removing group A from AB entity it's still in group A", GroupsNoA.ContainsType(GroupTypeA));

			EntityManager->RemoveEntityFromGroupType(Entities[0], GroupTypeB);
			const FMassArchetypeHandle ArchetypeNoGroups = EntityManager->GetArchetypeForEntity(Entities[0]);
			AITEST_EQUAL("The archetype after removing the entity from both groups; and the original archetype, scenario 1", ArchetypeNoGroups, OriginalArchetype);
		}
		{
			// let's do the same in the other order, first remove B then A
			EntityManager->RemoveEntityFromGroupType(Entities[1], GroupTypeB);
			const FMassArchetypeHandle ArchetypeGroupNoB = EntityManager->GetArchetypeForEntity(Entities[1]);

			const FArchetypeGroups& GroupsNoB = EntityManager->GetGroupsForArchetype(ArchetypeGroupNoB);
			AITEST_FALSE("After removing group B from AB entity it's still in group B", GroupsNoB.ContainsType(GroupTypeB));
			AITEST_TRUE("After removing group B from AB entity it's still in group A", GroupsNoB.ContainsType(GroupTypeA));

			EntityManager->RemoveEntityFromGroupType(Entities[1], GroupTypeA);
			const FMassArchetypeHandle ArchetypeNoGroups = EntityManager->GetArchetypeForEntity(Entities[1]);
			AITEST_EQUAL("The archetype after removing the entity from both groups; and the original archetype, scenario 2", ArchetypeNoGroups, OriginalArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeGroup_Remove, "System.Mass.Archetype.Group.Remove");

} // UE::Mass::Test

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
