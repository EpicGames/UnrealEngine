// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityBuilder.h"
#include "MassRelationCommands.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "Algo/Compare.h"
#include "Relations/MassChildOf.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::ChildOfRelation
{
	using namespace UE::Mass;

	static TNotNull<UScriptStruct*> MakeRelationType(FName TypeName)
	{
		UScriptStruct* NewStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(), FName(*FString::Printf(TEXT("Relation_%s"), *TypeName.ToString())), RF_Public);
		NewStruct->SetSuperStruct(FMassRelation::StaticStruct());

		return NewStruct;
	}

	struct FChildOfBase : FEntityTestBase
	{
		enum class EStructure : uint8
		{
			String,
			Tree,
			MAX
		};
		EStructure StructureType = EStructure::MAX;

		using Super = FEntityTestBase;
		TArray<FMassEntityHandle> CreatedEntities;
		int32 NumEntities = -1;
		UMassTestProcessorBase* Processor = nullptr;
		bool bAPISupportsReparenting = true;
		FTypeHandle RelationTypeHandle;
		UE::Mass::FRelationManager* RelationManager = nullptr;

		virtual void BuildHierarchy() = 0;
		virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex)
		{
			return false;
		}
		virtual bool DeleteEntity(const int32 EntityIndex)
		{
			EntityManager->DestroyEntity(CreatedEntities[EntityIndex]);
			return true;
		}

		virtual void SetUpRelationHandle()
		{
			RelationTypeHandle = EntityManager->GetTypeManager().GetRelationTypeHandle(FMassChildOfRelation::StaticStruct());
			ensure(RelationTypeHandle == UE::Mass::Relations::ChildOfHandle);
		}

		virtual bool SetUp() override
		{
			check(StructureType != EStructure::MAX);
			switch (StructureType)
			{
			case EStructure::String:
				NumEntities = 5;
				break;
			case EStructure::Tree:
				NumEntities = 8;
				break;
			}

			if (Super::SetUp())
			{
				SetUpRelationHandle();
				RelationManager = &EntityManager->GetRelationManager();

				Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
				Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
				Processor->EntityQuery.AddRequirement<FMassChildOfFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
				Processor->EntityQuery.GroupBy(EntityManager->GetTypeManager().GetRelationTypeChecked(RelationTypeHandle).RegisteredGroupType);

				BuildHierarchy();

				return true;
			}
			return false;
		}

		virtual void ExecuteTestProcessor(TArray<int32>& Scratchpad)
		{
			Scratchpad.Reset();
			Scratchpad.AddUninitialized(CreatedEntities.Num());
			for (int32 EntityIndex = 0; EntityIndex < CreatedEntities.Num(); ++EntityIndex)
			{
				Scratchpad[EntityIndex] = -1;
			}

			EntityManager->FlushCommands();

			Processor->ForEachEntityChunkExecutionFunction = [RelationManager = RelationManager, CreatedEntitiesView = MakeArrayView(CreatedEntities), &Scratchpad](FMassExecutionContext& Context)
			{
				// @todo should this be the right way to fetch parents?
				TConstArrayView<FMassChildOfFragment> Parents = Context.GetFragmentView<FMassChildOfFragment>();
				if (Parents.Num())
				{
					for (auto EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						// get parent
						const FMassEntityHandle ParentHandle = Parents[EntityIt].Parent;
						if (ParentHandle.IsValid())
						{
							const int32 EntityGlobalIndex = CreatedEntitiesView.Find(Context.GetEntity(EntityIt));
							const int32 ParentGlobalIndex = CreatedEntitiesView.Find(ParentHandle);
							check(EntityGlobalIndex != INDEX_NONE && ParentGlobalIndex != INDEX_NONE);
							Scratchpad[EntityGlobalIndex] = Scratchpad[ParentGlobalIndex] * 10 + EntityGlobalIndex;
						}
					}
				}
				else
				{
					// no-parent
					for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
					{
						const int32 EntityGlobalIndex = CreatedEntitiesView.Find(Context.GetEntity(EntityIndex));
						check(EntityGlobalIndex != INDEX_NONE);
						Scratchpad[EntityGlobalIndex] = EntityGlobalIndex;
					}
				}
			};

			Processor->TestExecute(EntityManager);
		}

		virtual bool InstantTest() override
		{
			TArray<int32> Scratchpad;
			ExecuteTestProcessor(Scratchpad);

			TArray<int32> ExpectedValues;

			switch (StructureType)
			{
			case EStructure::String:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	L	1
				//		L	2
				//			L	3
				//				L	4
				ExpectedValues = { 0, 1, 12, 123, 1234 };
				break;

			case EStructure::Tree:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	+	1
				//	|	+	3
				//	|	+	4
				//	|	L	5
				//	L	2
				//		+	6
				//		L	7
				ExpectedValues = { 0, 1, 2, 13, 14, 15, 26, 27 };
				break;
			}
			AITEST_TRUE("Processing hierarchy produces expected results", Algo::Compare(Scratchpad, ExpectedValues));

			if (bAPISupportsReparenting == false)
			{
				// the API is just for entity creation, we end the test here.
				return true;
			}

			// here we reparent a leaf node
			AITEST_TRUE("Reparenting leaf node.", ReparentEntity(CreatedEntities.Num() - 1, 0));
			switch (StructureType)
			{
			case EStructure::String:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	L	1
				//	|	L	2
				//	|		L	3
				//	L	4
				ExpectedValues = { 0, 1, 12, 123, 4 };
				break;

			case EStructure::Tree:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	+	1
				//	|	+	3
				//	|	+	4
				//	|	L	5
				//	L	2
				//	|	L	6
				//	L	7
				ExpectedValues = { 0, 1, 2, 13, 14, 15, 26, 7 };
				break;
			}
			ExecuteTestProcessor(Scratchpad);
			AITEST_TRUE("Reparenting a leaf produces expected results", Algo::Compare(Scratchpad, ExpectedValues));

			AITEST_TRUE("Reparenting a mid-node to a leaf node.", ReparentEntity(2, 4));
			switch (StructureType)
			{
			case EStructure::String:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	L	1
				//	L	4
				//		L	2
				//			L	3
				ExpectedValues = { 0, 1, 42, 423, 4 };
				break;

			case EStructure::Tree:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	+	1
				//	|	+	3
				//	|	+	4
				//	|	|	L	2
				//	|	|		L	6
				//	|	L	5
				//	L	7
				ExpectedValues = { 0, 1, 142, 13, 14, 15, 1426, 7 };
				break;
			}
			ExecuteTestProcessor(Scratchpad);
			AITEST_TRUE("Reparenting a subtree to a leaf produces expected results", Algo::Compare(Scratchpad, ExpectedValues));

			AITEST_TRUE("Deleting a leaf node", DeleteEntity(3));
			switch (StructureType)
			{
			case EStructure::String:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	L	1
				//	L	4
				//		L	2
				ExpectedValues = { 0, 1, 42, -1, 4 };
				break;

			case EStructure::Tree:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	+	1
				//	|	+	4
				//	|	|	L	2
				//	|	|		L	6
				//	|	L	5
				//	L	7
				ExpectedValues = { 0, 1, 142, -1, 14, 15, 1426, 7 };
				break;
			}
			ExecuteTestProcessor(Scratchpad);
			AITEST_TRUE("Delete a leaf node produces expected results", Algo::Compare(Scratchpad, ExpectedValues));

			AITEST_TRUE("Deleting mid-level node", DeleteEntity(4));
			switch (StructureType)
			{
			case EStructure::String:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	+	1
				ExpectedValues = { 0, 1, -1, -1, -1 };
				break;

			case EStructure::Tree:
				// requires SetUp() override to implement the following structure, indicated with indices to CreatedEntities:
				//	0
				//	+	1
				//	|	L	5
				//	L	7
				ExpectedValues = { 0, 1, -1, -1, -1, 15, -1, 7 };
				break;
			}
			ExecuteTestProcessor(Scratchpad);
			AITEST_TRUE("Delete a mid-level node produces expected results", Algo::Compare(Scratchpad, ExpectedValues));

			AITEST_TRUE("Deleting mid-level node", DeleteEntity(0));
			switch (StructureType)
			{
			case EStructure::String:
				ExpectedValues = { -1, -1, -1, -1, -1 };
				break;

			case EStructure::Tree:
				ExpectedValues = { -1, -1, -1, -1, -1, -1, -1, -1 };
				break;
			}
			ExecuteTestProcessor(Scratchpad);
			AITEST_TRUE("Delete a top-level node produces expected results", Algo::Compare(Scratchpad, ExpectedValues));

			return true;
		}
	};

	struct FChildOf_IndividualAPI_StringHierarchy : FChildOfBase
	{
		FChildOf_IndividualAPI_StringHierarchy()
		{
			StructureType = EStructure::String;
		}

		virtual void BuildHierarchy() override
		{
			EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
			// set up indices
			for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
			{
				FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
				IndexCounterFragment.Value = Index;
			}

			for (int32 ChildIndex = 1; ChildIndex < CreatedEntities.Num(); ++ChildIndex)
			{
				RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[ChildIndex], CreatedEntities[ChildIndex - 1]);
			}
		}

		virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
		{
			return RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]).IsValid();
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOf_IndividualAPI_StringHierarchy, "System.Mass.Relations.ChildOf.IndividualAPI.StringHierarchy");

	struct FChildOf_IndividualAPI_TreeHierarchy : FChildOfBase
	{
		FChildOf_IndividualAPI_TreeHierarchy()
		{
			StructureType = EStructure::Tree;
		}

		virtual void BuildHierarchy() override
		{
			EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
			// set up indices
			for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
			{
				FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
				IndexCounterFragment.Value = Index;
			}

			// [1] - child of [0]
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[1], CreatedEntities[0]);
			// [2] - child of [0]
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[2], CreatedEntities[0]);
			// [3] - child of [1]
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[3], CreatedEntities[1]);
			// [4] - child of [1]
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[4], CreatedEntities[1]);
			// [5] - child of [1]
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[5], CreatedEntities[1]);
			// [6] - child of [2]
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[6], CreatedEntities[2]);
			// [7] - child of [2]
			RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[7], CreatedEntities[2]);
		};

		virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
		{
			return RelationManager->CreateRelationInstance(RelationTypeHandle, CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]).IsValid();
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOf_IndividualAPI_TreeHierarchy, "System.Mass.Relations.ChildOf.IndividualAPI.TreeHierarchy");

	struct FChildOfEntityBuilder_StringHierarchy : FChildOfBase
	{
		FChildOfEntityBuilder_StringHierarchy()
		{
			StructureType = EStructure::String;
			bAPISupportsReparenting = false;
		}

		virtual void BuildHierarchy() override
		{
			FEntityBuilder Builder = EntityManager->MakeEntityBuilder();

			FTestFragment_Int& IndexCounterFragment = Builder.Add_GetRef<FTestFragment_Int>(CreatedEntities.Num());
			CreatedEntities.Add(Builder.CommitAndReprepare());

			for (int32 ChildIndex = 0; ChildIndex < 4; ++ChildIndex)
			{
				Builder.AddRelation(RelationTypeHandle, CreatedEntities.Last());
				IndexCounterFragment.Value = CreatedEntities.Num();
				CreatedEntities.Add(Builder.CommitAndReprepare());
			}
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOfEntityBuilder_StringHierarchy, "System.Mass.Relations.ChildOf.EntityBuilder.StringHierarchy");

	struct FChildOfEntityBuilder_TreeHierarchy : FChildOfBase
	{
		FChildOfEntityBuilder_TreeHierarchy()
		{
			StructureType = EStructure::Tree;
			bAPISupportsReparenting = false;
		}

		virtual void BuildHierarchy() override
		{
			FEntityBuilder Builder = EntityManager->MakeEntityBuilder();

			FTestFragment_Int& IndexCounterFragment = Builder.Add_GetRef<FTestFragment_Int>(CreatedEntities.Num());
			// [0] - parent entity
			CreatedEntities.Add(Builder.CommitAndReprepare());

			Builder.AddRelation(RelationTypeHandle, CreatedEntities.Last());
			// [1] - child of [0]
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());
			// [2] - child of [0]
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());

			Builder.AddRelation(RelationTypeHandle, CreatedEntities[1]);
			// [3] - child of [1]
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());
			// [4] - child of [1]
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());
			// [5] - child of [1]
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());

			Builder.AddRelation(RelationTypeHandle, CreatedEntities[2]);
			// [6] - child of [2]
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());
			// [7] - child of [2]
			IndexCounterFragment.Value = CreatedEntities.Num();
			CreatedEntities.Add(Builder.CommitAndReprepare());
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOfEntityBuilder_TreeHierarchy, "System.Mass.Relations.ChildOf.EntityBuilder.TreeHierarchy");

	struct FChildOfBatchAPI_Tree : FChildOfBase
	{
		FChildOfBatchAPI_Tree()
		{
			StructureType = EStructure::Tree;
		}

		virtual void BuildHierarchy() override
		{
			EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
			// set up indices
			for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
			{
				FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
				IndexCounterFragment.Value = Index;
			}

			// [1, 2] - child of [0]
			EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[1], 1), MakeArrayView(&CreatedEntities[0], 1));
			EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[2], 1), MakeArrayView(&CreatedEntities[0], 1));
			// [3, 4, 5] - child of [1]
			EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[3], 1), MakeArrayView(&CreatedEntities[1], 1));
			EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[4], 1), MakeArrayView(&CreatedEntities[1], 1));
			EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[5], 1), MakeArrayView(&CreatedEntities[1], 1));
			// [6, 7] - child of [2]
			EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[6], 1), MakeArrayView(&CreatedEntities[2], 1));
			EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[7], 1), MakeArrayView(&CreatedEntities[2], 1));
		}

		virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
		{
			return EntityManager->BatchCreateRelations(RelationTypeHandle, MakeArrayView(&CreatedEntities[ChildIndex], 1), MakeArrayView(&CreatedEntities[ParentIndex], 1));
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOfBatchAPI_Tree, "System.Mass.Relations.ChildOf.BatchAPI");

	struct FChildOfCommands_StringHierarchy : FChildOfBase
	{
		FChildOfCommands_StringHierarchy()
		{
			StructureType = EStructure::String;
		}

		virtual void BuildHierarchy() override
		{
			EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
			// set up indices
			for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
			{
				FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
				IndexCounterFragment.Value = Index;
				if (Index > 0)
				{
					// issue relationship-creating commands, done here for convenience 
					EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(CreatedEntities[Index], CreatedEntities[Index-1]);
				}
			}

			EntityManager->FlushCommands();
		}

		virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
		{
			EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]);
			return true;
		}

		virtual bool DeleteEntity(const int32 EntityIndex) override
		{
			EntityManager->Defer().DestroyEntity(CreatedEntities[EntityIndex]);
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOfCommands_StringHierarchy, "System.Mass.Relations.ChildOf.Commands.StringHierarchy");

	struct FChildOfCommands_TreeHierarchy : FChildOfBase
	{
		FChildOfCommands_TreeHierarchy()
		{
			StructureType = EStructure::Tree;
		}

		virtual void BuildHierarchy() override
		{
			EntityManager->BatchCreateEntities(IntsArchetype, {}, NumEntities, CreatedEntities);
			
			for (int32 Index = 0; Index < CreatedEntities.Num(); ++Index)
			{
				FTestFragment_Int& IndexCounterFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CreatedEntities[Index]);
				IndexCounterFragment.Value = Index;
			}

			// [1, 2] - child of [0]
			EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(MakeArrayView(&CreatedEntities[1], 2), MakeArrayView(&CreatedEntities[0], 1));
			// [3, 4, 5] - child of [1]
			EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(MakeArrayView(&CreatedEntities[3], 3), MakeArrayView(&CreatedEntities[1], 1));
			// [6, 7] - child of [2]
			EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(MakeArrayView(&CreatedEntities[6], 2), MakeArrayView(&CreatedEntities[2], 1));

			EntityManager->FlushCommands();
		}

		virtual bool ReparentEntity(const int32 ChildIndex, const int32 ParentIndex) override
		{
			EntityManager->Defer().PushCommand<FMassCommandMakeRelation<FMassChildOfRelation>>(CreatedEntities[ChildIndex], CreatedEntities[ParentIndex]);
			EntityManager->FlushCommands();
			return true;
		}

		virtual bool DeleteEntity(const int32 EntityIndex) override
		{
			EntityManager->Defer().DestroyEntity(CreatedEntities[EntityIndex]);
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOfCommands_TreeHierarchy, "System.Mass.Relations.ChildOf.Commands.TreeHierarchy");

	struct FChildOfBatchAPI_Tree_NoMapping : FChildOfBatchAPI_Tree
	{
		virtual void SetUpRelationHandle() override
		{
			UScriptStruct* MaplessRelationType = MakeRelationType("ChildOfNoMapping");
			FRelationTypeTraits RelationTraits(EntityManager->GetTypeManager().GetRelationTypeChecked(UE::Mass::Relations::ChildOfHandle), MaplessRelationType);
			RelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].RequiresExternalMapping = EExternalMappingRequired::No;

			CA_ASSUME(MaplessRelationType);
			RelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(RelationTraits));
		}
	};
	// commented out on purpose, functionality not implemented yet
	//IMPLEMENT_AI_INSTANT_TEST(FChildOfBatchAPI_Tree_NoMapping, "System.Mass.Relations.ChildOf.NoMapping.BatchAPI");

	// - other tests ---
	struct FSetChildAsParent : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			FTypeHandle RelationTypeHandle = EntityManager->GetTypeManager().GetRelationTypeHandle(FMassChildOfRelation::StaticStruct());

			TArray<FMassEntityHandle> CreatedEntities;
			EntityManager->BatchCreateEntities(IntsArchetype, {}, 2, CreatedEntities);

			UE::Mass::FRelationManager& RelationManager = EntityManager->GetRelationManager();
			RelationManager.CreateRelationInstance(RelationTypeHandle, CreatedEntities[0], CreatedEntities[1]);
			EntityManager->FlushCommands();

			// original parent becomes the child and vice versa:
			RelationManager.CreateRelationInstance(RelationTypeHandle, CreatedEntities[1], CreatedEntities[0]);
			EntityManager->FlushCommands();

			TArray<FMassEntityHandle> ParentSubjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[0]);
			TArray<FMassEntityHandle> ParentObjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[0]);
			TArray<FMassEntityHandle> ChildSubjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[1]);
			TArray<FMassEntityHandle> ChildObjects = RelationManager.GetRelationSubjects(RelationTypeHandle, CreatedEntities[1]);

			//AITEST_TRUE()

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSetChildAsParent, "System.Mass.Relations.ChildOf.SetChildAsParent");


} // UE::Mass::Test::ChildOfRelation

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
#undef WITH_BUILDER