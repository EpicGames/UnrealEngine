// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassArchetypeData.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "Algo/RandomShuffle.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"
#include "Algo/Compare.h"
#include "Algo/RemoveIf.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//
namespace FMassArchetypeEntityCollectionTest
{

struct FArchetypeEntityCollectionTestBase : FEntityTestBase
{
	TArray<FMassEntityHandle> Entities;

	virtual bool SetUp() override
	{
		FEntityTestBase::SetUp();

		const int32 Count = 100;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);
		return true;
	}

	virtual void TearDown() override
	{
		Entities.Reset();
		FEntityTestBase::TearDown();
	}
};

struct FArchetypeEntityCollection_CreateBasic : FArchetypeEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> EntitiesSubSet;
		
		// Should end up as last chunk
		EntitiesSubSet.Add(Entities[99]);
		EntitiesSubSet.Add(Entities[97]);
		EntitiesSubSet.Add(Entities[98]);

		// Should end up as third chunk
		EntitiesSubSet.Add(Entities[20]);
		EntitiesSubSet.Add(Entities[22]);
		EntitiesSubSet.Add(Entities[21]);

		// Should end up as second chunk
		EntitiesSubSet.Add(Entities[18]);

		// Should end up as first chunk
		EntitiesSubSet.Add(Entities[10]);
		EntitiesSubSet.Add(Entities[13]);
		EntitiesSubSet.Add(Entities[11]);
		EntitiesSubSet.Add(Entities[12]);

		FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);
		FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
		AITEST_EQUAL("The predicted sub-chunk count should match", Ranges.Num(), 4);
		AITEST_EQUAL("The [10-13] chunk should be first and start at 10", Ranges[0].SubchunkStart, 10);
		AITEST_EQUAL("The [10-13] chunk should be first and have a length of 4", Ranges[0].Length, 4);
		AITEST_EQUAL("The [18] chunk should be second and start at 18", Ranges[1].SubchunkStart, 18); 
		AITEST_EQUAL("The [18] chunk should be second and have a length of 1", Ranges[1].Length, 1);
		AITEST_EQUAL("The [20-22] chunk should be third and start at 20", Ranges[2].SubchunkStart, 20);
		AITEST_EQUAL("The [20-22] chunk should be third and have a length of 3", Ranges[2].Length, 3);
		AITEST_EQUAL("The [97-99] chunk should be third and start at 97", Ranges[3].SubchunkStart, 97);
		AITEST_EQUAL("The [97-99] chunk should be third and have a length of 3", Ranges[3].Length, 3);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_CreateBasic, "System.Mass.ArchetypeEntityCollection.Create.Basic");

struct FArchetypeEntityCollection_CreateOrderInvariant : FArchetypeEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> EntitiesSubSet(&Entities[10], 30);
		EntitiesSubSet.RemoveAt(10, EAllowShrinking::No);

		FMassArchetypeEntityCollection CollectionFromOrdered(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);

		FRandomStream Rand(0);
		ShuffleDataWithRandomStream(Rand, EntitiesSubSet);

		FMassArchetypeEntityCollection CollectionFromRandom(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);

		AITEST_TRUE("The resulting chunk collection should be the same regardless of the order of input entities", CollectionFromOrdered.IsSame(CollectionFromRandom));
		
		// just to roughly make sure the result is what we expect
		FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = CollectionFromOrdered.GetRanges();
		AITEST_EQUAL("The result should contain two chunks", Ranges.Num(), 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_CreateOrderInvariant, "System.Mass.ArchetypeEntityCollection.Create.OrderInvariant");

struct FArchetypeEntityCollection_CreateCrossChunk : FEntityTestBase
{
	virtual bool InstantTest() override
	{
#if WITH_MASSENTITY_DEBUG
		TArray<FMassEntityHandle> Entities;
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);

		const int32 SpillOver = 10;
		const int32 Count = EntitiesPerChunk + SpillOver;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);
		
		TArray<FMassEntityHandle> EntitiesSubCollection;
		EntitiesSubCollection.Add(Entities[EntitiesPerChunk]);
		for (int i = 1; i < SpillOver; ++i)
		{
			EntitiesSubCollection.Add(Entities[EntitiesPerChunk + i]);
			EntitiesSubCollection.Add(Entities[EntitiesPerChunk - i]);
		}
		
		FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesSubCollection, FMassArchetypeEntityCollection::NoDuplicates);
		FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
		AITEST_EQUAL("The given continuous range should get split in two", Ranges.Num(), 2);
		AITEST_EQUAL("The part in first archetype\'s chunk should contain 9 elements", Ranges[0].Length, 9);
		AITEST_EQUAL("The part in second archetype\'s chunk should contain 10 elements", Ranges[1].Length, 10);
#endif // WITH_MASSENTITY_DEBUG
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_CreateCrossChunk, "System.Mass.ArchetypeEntityCollection.Create.CrossChunk");

struct FArchetypeEntityCollection_CreateWithDuplicatesTrivial : FArchetypeEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> EntitiesWithDuplicates;
		EntitiesWithDuplicates.Add(Entities[2]);
		EntitiesWithDuplicates.Add(Entities[2]);

		FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesWithDuplicates, FMassArchetypeEntityCollection::FoldDuplicates);
		FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
		AITEST_EQUAL("The result should have a single subchunk", Ranges.Num(), 1);
		AITEST_EQUAL("The resulting subchunk should be of length 1", Ranges[0].Length, 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_CreateWithDuplicatesTrivial, "System.Mass.ArchetypeEntityCollection.Create.TrivialDuplicates");

struct FArchetypeEntityCollection_CreateWithDuplicates : FArchetypeEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> EntitiesWithDuplicates;
		EntitiesWithDuplicates.Add(Entities[0]);
		EntitiesWithDuplicates.Add(Entities[0]);
		EntitiesWithDuplicates.Add(Entities[0]);
		EntitiesWithDuplicates.Add(Entities[1]);
		EntitiesWithDuplicates.Add(Entities[2]);
		EntitiesWithDuplicates.Add(Entities[2]);

		FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesWithDuplicates, FMassArchetypeEntityCollection::FoldDuplicates);
		FMassArchetypeEntityCollection::FConstEntityRangeArrayView Ranges = EntityCollection.GetRanges();
		AITEST_EQUAL("The result should have a single subchunk", Ranges.Num(), 1);
		AITEST_EQUAL("The resulting subchunk should be of length 3", Ranges[0].Length, 3);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_CreateWithDuplicates, "System.Mass.ArchetypeEntityCollection.Create.Duplicates");

struct FArchetypeEntityCollection_CreateWithInvalidDuplicates : FArchetypeEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		{
			TArray<FMassEntityHandle> EntitiesSubSet;

			EntitiesSubSet.Add(FMassEntityHandle());
			EntitiesSubSet.Add(Entities[0]);
			EntitiesSubSet.Add(Entities[0]);
			EntitiesSubSet.Add(FMassEntityHandle());

			FMassArchetypeEntityCollection Collection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates);

			// The resulting Collection should have only a single Range consisting of a single entity (matching Entities[0])
			AITEST_EQUAL(TEXT("We expect only a single resulting range"), Collection.GetRanges().Num(), 1);
			AITEST_EQUAL(TEXT("We expect only a single entity in the resulting range"), Collection.GetRanges()[0].SubchunkStart, 0);
			AITEST_EQUAL(TEXT("We expect only a single entity in the resulting range"), Collection.GetRanges()[0].Length, 1);
		}

		{
			TArray<FMassEntityHandle> EntitiesSubSet;

			EntitiesSubSet.Add(Entities[4]);
			EntitiesSubSet.Add(FMassEntityHandle());
			EntitiesSubSet.Add(FMassEntityHandle()); 
			EntitiesSubSet.Add(Entities[3]);
			EntitiesSubSet.Add(FMassEntityHandle());
			EntitiesSubSet.Add(Entities[1]);

			FMassArchetypeEntityCollection Collection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates);

			// The resulting Collection should have two Ranges for a single archetype, one of them with two entities (3, 4).
			AITEST_EQUAL(TEXT("We expect two resulting range"), Collection.GetRanges().Num(), 2);
			AITEST_EQUAL(TEXT("We expect the first range to consist of a single entity"), Collection.GetRanges()[0].Length, 1);
			AITEST_EQUAL(TEXT("We expect the second range to consist of a two entities"), Collection.GetRanges()[1].Length, 2);
			// the specific composition of resulting ranges is being tested by other tests
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_CreateWithInvalidDuplicates, "System.Mass.ArchetypeEntityCollection.Create.InvalidDuplicates");

struct FArchetypeEntityCollection_CreateWithInvalidDuplicatesWithPayload : FArchetypeEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> EntitiesSubSet;
		TArray<FTestFragment_Int> Payload;

		EntitiesSubSet.Add(FMassEntityHandle());
		Payload.Add(FTestFragment_Int(int32(2)));

		EntitiesSubSet.Add(Entities[0]);
		Payload.Add(FTestFragment_Int(int32(0)));

		EntitiesSubSet.Add(Entities[0]);
		Payload.Add(FTestFragment_Int(int32(1)));

		EntitiesSubSet.Add(FMassEntityHandle());
		Payload.Add(FTestFragment_Int(int32(3)));

		// transform typed payload array into generic one for sorting purposes
		FStructArrayView PaloadView(Payload);
		TArray<FMassArchetypeEntityCollectionWithPayload> Result;
		{
			MASS_SCOPED_ENSURE_TEST("Invalid entity handle passed in", 2);
			FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates
				, FMassGenericPayloadView(MakeArrayView(&PaloadView, 1)), Result);
		}

		AITEST_EQUAL(TEXT("We expect only a single result"), Result.Num(), 1);
		AITEST_EQUAL(TEXT("We expect only a single resulting range"), Result[0].GetEntityCollection().GetRanges().Num(), 1);
		AITEST_EQUAL(TEXT("We expect only a single entity in the resulting range"), Result[0].GetEntityCollection().GetRanges()[0].SubchunkStart, 0);
		AITEST_EQUAL(TEXT("We expect only a single entity in the resulting range"), Result[0].GetEntityCollection().GetRanges()[0].Length, 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_CreateWithInvalidDuplicatesWithPayload, "System.Mass.ArchetypeEntityCollection.Create.InvalidDuplicatesWithPayloadWithPayload");

#if WITH_MASSENTITY_DEBUG
struct FArchetypeEntityCollection_WithPayloadBase : FArchetypeEntityCollectionTestBase
{
	virtual bool SetUp() override
	{
		// skipping FEntityCollectionTestBase::SetUp on purpose to manually create entities
		FEntityTestBase::SetUp();

		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
		EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesPerChunk * 2, Entities);
		return true;
	}
};

struct FArchetypeEntityCollection_TrivialDuplicatesWithPayload : FArchetypeEntityCollection_WithPayloadBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
		const int32 TotalCount = Entities.Num();
		const int32 SubSetCount = int(0.6 * Entities.Num()); // using >0.5 to ensure some entities picked being in sequence and/or in different chunks
		TArray<FMassEntityHandle> EntitiesSubSet;
		TArray<FTestFragment_Int> Payload;

		EntitiesSubSet.Add(Entities[EntitiesPerChunk + 20]); 
		Payload.Add(FTestFragment_Int(int32(0)));
		EntitiesSubSet.Add(Entities[EntitiesPerChunk + 20]);
		Payload.Add(FTestFragment_Int(int32(1)));

		// transform typed payload array into generic one for sorting purposes
		FStructArrayView PaloadView(Payload);
		TArray<FMassArchetypeEntityCollectionWithPayload> Result;
		FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates
			, FMassGenericPayloadView(MakeArrayView(&PaloadView, 1)), Result);

		AITEST_EQUAL(TEXT("We expect only a single result"), Result.Num(), 1);
		AITEST_EQUAL(TEXT("We expect only a single resulting range"), Result[0].GetEntityCollection().GetRanges().Num(), 1);
		AITEST_EQUAL(TEXT("We expect only a single entity in the resulting range"), Result[0].GetEntityCollection().GetRanges()[0].Length, 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_TrivialDuplicatesWithPayload, "System.Mass.ArchetypeEntityCollection.Create.TrivialDuplicatesWithPayload");

// @todo also add another archetype
struct FArchetypeEntityCollection_MultiDuplicatesWithPayload : FArchetypeEntityCollection_WithPayloadBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
		const int32 TotalCount = Entities.Num();
		const int32 SubSetCount = int(0.6 * Entities.Num()); // using >0.5 to ensure some entities picked being in sequence and/or in different chunks
		TArray<FMassEntityHandle> EntitiesSubSet;
		TArray<FTestFragment_Int> Payload;

		constexpr int32 NumUniques = 3;
		constexpr int32 NumDuplicatesEach = 4;
		int32 FragmentValue = 0;
		for (int32 Iteration = 0; Iteration < NumDuplicatesEach; ++Iteration, ++FragmentValue)
		{
			for (int32 Unique = 0; Unique < NumUniques; ++Unique)
			{
				EntitiesSubSet.Add(Entities[EntitiesPerChunk + 20 + Unique]);
				Payload.Add(FTestFragment_Int(Unique));
			}
		}

		// transform typed payload array into generic one for sorting purposes
		FStructArrayView PaloadView(Payload);
		TArray<FMassArchetypeEntityCollectionWithPayload> Result;
		FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates
			, FMassGenericPayloadView(MakeArrayView(&PaloadView, 1)), Result);

		AITEST_EQUAL(TEXT("We expect only a single result"), Result.Num(), 1);
		AITEST_EQUAL(TEXT("We expect only a single resulting range"), Result[0].GetEntityCollection().GetRanges().Num(), 1);
		AITEST_EQUAL(TEXT("We expect exatly NumUniques entities in the resulting range"), Result[0].GetEntityCollection().GetRanges()[0].Length, NumUniques);
		const FMassGenericPayloadViewSlice& PayloadSlice = Result[0].GetPayload();
		for (int32 Unique = 0; Unique < NumUniques; ++Unique)
		{
			AITEST_EQUAL("The surviving payload value should match the expected", PayloadSlice[0].GetAt<FTestFragment_Int>(Unique).Value, Unique);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_MultiDuplicatesWithPayload, "System.Mass.ArchetypeEntityCollection.Create.MultiDuplicatesWithPayload");

struct FArchetypeEntityCollection_WithPayload : FArchetypeEntityCollection_WithPayloadBase
{
	virtual bool InstantTest() override
	{
		const int32 TotalCount = Entities.Num();
		const int32 SubSetCount = int(0.6 * Entities.Num()); // using >0.5 to ensure some entities picked being in sequence and/or in different chunks
		TArray<FMassEntityHandle> EntitiesSubSet;
		TArray<FTestFragment_Int> Payload;

		TArray<int32> Indices;
		Indices.AddUninitialized(TotalCount);
		for (int32 i = 0; i < Indices.Num(); ++i)
		{
			Indices[i] = i;
		}

		FMath::SRandInit(TotalCount);
		Algo::RandomShuffle(Indices);
		Indices.SetNum(SubSetCount);

		for (int32 i : Indices)
		{
			EntitiesSubSet.Add(Entities[i]);
			Payload.Add(FTestFragment_Int(i));
		}

		// transform typed payload array into generic one for sorting purposes
		FStructArrayView PaloadView(Payload);
		TArray<FMassArchetypeEntityCollectionWithPayload> Result;
		FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, EntitiesSubSet, FMassArchetypeEntityCollection::FoldDuplicates
			, FMassGenericPayloadView(MakeArrayView(&PaloadView, 1)), Result);

		// at this point Payload should be sorted ascending 
		// and the values in Payload should match entities at given locations
		for (int i = 1; i < Payload.Num(); ++i)
		{
			AITEST_TRUE("Items in Payload should be arranged in an ascending manner", Payload[i].Value >= Payload[i-1].Value);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_WithPayload, "System.Mass.ArchetypeEntityCollection.Create.WithPayload");

struct FArchetypeEntityCollection_ZeroLength_Destroy : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
		const int32 NumChunks = 3;
		const int32 EntityCount = NumChunks * EntitiesPerChunk;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);

		AITEST_EQUAL("Entity creation generated expected number of entities", EntityCount, FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());

		FMassEntityQuery Query(EntityManager);
		Query.CacheArchetypes();
		FMassArchetypeEntityCollection Collection(SourceArchetype);
		AITEST_EQUAL("Created collection has expected number of ranges (matching number of chunks created"
			, Collection.GetRanges().Num(), NumChunks);
		
		EntityManager->BatchDestroyEntityChunks(Collection);
		AITEST_EQUAL("Entity destruction destroyed all entities", FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities(), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_ZeroLength_Destroy, "System.Mass.Entity.ZeroLengthRanges.Destroy");

struct FArchetypeEntityCollection_ZeroLength_Execute : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
		const int32 NumChunks = 3;
		const int32 EntityCount = NumChunks * EntitiesPerChunk;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);
		FMassArchetypeEntityCollection Collection(SourceArchetype);

		AITEST_EQUAL("Entity creation generated expected number of entities", EntityCount, FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());

		int32 ProcessedEntitiesCount = 0;
		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		Processor->ForEachEntityChunkExecutionFunction = [&ProcessedEntitiesCount](FMassExecutionContext& Context)
		{
			ProcessedEntitiesCount += Context.GetNumEntities();
		};
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
		UE::Mass::Executor::RunProcessorsView(MakeArrayView((UMassProcessor**)&Processor, 1), ProcessingContext, MakeConstArrayView(&Collection, 1));

		AITEST_EQUAL("Number of entities processed matches expectations"
			, ProcessedEntitiesCount, EntityCount);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_ZeroLength_Execute, "System.Mass.Entity.ZeroLengthRanges.Execute");

struct FArchetypeEntityCollection_ZeroLength_BatchMove : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
		const int32 NumChunks = 3;
		const int32 EntityCount = NumChunks * EntitiesPerChunk;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);
		FMassArchetypeEntityCollection Collection(SourceArchetype);

		const UScriptStruct* AddedTag = FTestTag_A::StaticStruct();

		AITEST_EQUAL("Entity creation generated expected number of entities", EntityCount, FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());
		check(EntityCount == Entities.Num());

		EntityManager->BatchChangeTagsForEntities(MakeArrayView(&Collection, 1), FMassTagBitSet(AddedTag), {});

		const FMassArchetypeHandle NewArchetype = EntityManager->GetArchetypeForEntity(Entities[0]);
		AITEST_NOT_EQUAL("Entities have changed their host archetype", NewArchetype, SourceArchetype);

		const FMassArchetypeHandle ExpectedArchetype = EntityManager->CreateArchetype(SourceArchetype, MakeArrayView(&AddedTag, 1));
		AITEST_EQUAL("The new archetype matches expectations", NewArchetype, ExpectedArchetype);
		AITEST_EQUAL("All the entities have been moved to the expected archetype", EntityManager->DebugGetArchetypeEntitiesCount(NewArchetype), EntityCount);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_ZeroLength_BatchMove, "System.Mass.Entity.ZeroLengthRanges.BatchMove");

struct FArchetypeEntityCollection_ZeroLength_BatchSetFragmentValues : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
		const int32 NumChunks = 3;
		const int32 EntityCount = NumChunks * EntitiesPerChunk;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);

		AITEST_EQUAL("Entity creation generated expected number of entities", EntityCount, FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntities());
		check(EntityCount == Entities.Num());

		FMassArchetypeEntityCollectionWithPayload EntityCollectionWithEmptyPayload{FMassArchetypeEntityCollection(SourceArchetype)};

		const FMassFragmentBitSet AddedFragmentsBitSet(FTestFragment_Int::StaticStruct());
		EntityManager->BatchAddFragmentInstancesForEntities(MakeArrayView(&EntityCollectionWithEmptyPayload, 1), AddedFragmentsBitSet);

		const FMassArchetypeHandle NewArchetype = EntityManager->GetArchetypeForEntity(Entities[0]);
		AITEST_NOT_EQUAL("Entities have changed their host archetype", NewArchetype, SourceArchetype);

		const FMassArchetypeHandle ExpectedArchetype = FloatsIntsArchetype;
		AITEST_EQUAL("The new archetype matches expectations", NewArchetype, ExpectedArchetype);
		AITEST_EQUAL("All the entities have been moved to the expected archetype", EntityManager->DebugGetArchetypeEntitiesCount(NewArchetype), EntityCount);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_ZeroLength_BatchSetFragmentValues, "System.Mass.Entity.ZeroLengthRanges.BatchSetFragmentValues");

struct FArchetypeEntityCollection_ExportHandles_Created : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassArchetypeHandle SourceArchetype = FloatsArchetype;
		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype).GetNumEntitiesPerChunk();
		const int32 NumChunks = 3;
		const int32 EntityCount = NumChunks * EntitiesPerChunk;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(SourceArchetype, {}, EntityCount, Entities);

		// deterministically removing half of the entities
		FRandomStream RandomStream(0);
		const int32 RemainingCount = Algo::StableRemoveIf(Entities, [&RandomStream](const FMassEntityHandle& Element)
		{
			return RandomStream.FRand() < 0.5f;
		});
		Entities.SetNum(RemainingCount);

		FMassArchetypeEntityCollection Collection(SourceArchetype, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);

		TArray<FMassEntityHandle> ExportedHandles;
		Collection.ExportEntityHandles(ExportedHandles);

		AITEST_TRUE("Exported handles are the same as the input data", Algo::Compare(ExportedHandles, Entities));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FArchetypeEntityCollection_ExportHandles_Created, "System.Mass.ArchetypeEntityCollection.ExportHandles.CreatedEntities");

#endif // WITH_MASSENTITY_DEBUG
//
}

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
