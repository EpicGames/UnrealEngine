// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassCommands.h"
#include "MassEntityTestTypes.h"
#include "MassEntityTypes.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "Algo/RandomShuffle.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//

namespace FMassCommandsTest
{
#if WITH_MASSENTITY_DEBUG
struct FCommands_FragmentInstanceList : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		TArray<FMassEntityHandle> IntEntities;
		TArray<FMassEntityHandle> FloatEntities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, IntEntities);
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, FloatEntities);

		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(IntEntities[i], FTestFragment_Int(i), FTestFragment_Float((float)i));
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(FloatEntities[i], FTestFragment_Int(i), FTestFragment_Float((float)i));
		}

		EntityManager->FlushCommands();

		auto TestEntities = [this](const TArray<FMassEntityHandle>& Entities) -> bool {
			// all entities should have ended up in the same archetype, FloatsIntsArchetype
			for (int i = 0; i < Entities.Num(); ++i)
			{
				AITEST_EQUAL(TEXT("All entities should have ended up in the same archetype"), EntityManager->GetArchetypeForEntity(Entities[i]), FloatsIntsArchetype);

				FMassEntityView View(FloatsIntsArchetype, Entities[i]);
				AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Int>().Value, i);
				AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Float>().Value, float(i));
			}
			return true;
		};
		
		if (!TestEntities(IntEntities) || !TestEntities(FloatEntities))
		{
			return false;
		}
		//AITEST_EQUAL(TEXT("All entities should have ended up in the same archetype"), EntitySubsystem->GetArchetypeForEntity(FloatEntities[i]), FloatsIntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_FragmentInstanceList, "System.Mass.Commands.FragmentInstanceList");


struct FCommands_FragmentMemoryCleanup : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const UScriptStruct* ArrayFragmentTypes[] = {
			FTestFragment_Array::StaticStruct(), 
			FTestFragment_Int::StaticStruct()
		};
		const FMassArchetypeHandle ArrayArchetype = EntityManager->CreateArchetype(MakeArrayView(ArrayFragmentTypes, 1));
		const FMassArchetypeHandle ArrayIntArchetype = EntityManager->CreateArchetype(MakeArrayView(ArrayFragmentTypes, 2));
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(ArrayArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);
		
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(ArrayArchetype, Count, Entities);

		AITEST_EQUAL(TEXT("All entities created should be in ArrayArchetype"), EntityManager->DebugGetArchetypeEntitiesCount(ArrayArchetype), Entities.Num());

		TArray<int32> EntitiesWithArray;
		for (int EntityIndex = 0; EntityIndex < Count; ++EntityIndex)
		{
			if (FMath::FRand() < 0.2)
			{	
				FTestFragment_Array A;
				A.Value.Add(EntityIndex);
				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[EntityIndex], A);
				EntityManager->Defer().AddFragment<FTestFragment_Int>(Entities[EntityIndex]);
				EntitiesWithArray.Add(EntityIndex);
			}
		}

		EntityManager->FlushCommands();

		for (int32 EntityIndex : EntitiesWithArray)
		{
			FMassEntityView View(ArrayIntArchetype, Entities[EntityIndex]);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Array>().Value.Num(), 1);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Array>().Value[0], EntityIndex);
		}

		// not move things a round by adding yet another fragment. That will force moving of some array-hosting fragments
		for (int EntityIndex = 0; EntityIndex < Count; ++EntityIndex)
		{
			if (FMath::FRand() < 0.5)
			{	
				EntityManager->Defer().AddFragment<FTestFragment_Float>(Entities[EntityIndex]);
			}
		}

		EntityManager->FlushCommands();

		for (int32 EntityIndex : EntitiesWithArray)
		{
			FMassEntityView View(*EntityManager, Entities[EntityIndex]);
			AITEST_EQUAL(TEXT("Potentially moved array fragment should have predicted values"), View.GetFragmentData<FTestFragment_Array>().Value.Num(), 1);
			AITEST_EQUAL(TEXT("Potentially moved array fragment should have predicted values"), View.GetFragmentData<FTestFragment_Array>().Value[0], EntityIndex);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_FragmentMemoryCleanup, "System.Mass.Commands.MemoryManagement");

// @todo add "add-then remove some to make holes in chunks-then add again" test
struct FCommands_BuildEntitiesWithFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

		TArray<FMassEntityHandle> Entities;
		for (int i = 0; i < Count; ++i)
		{
			Entities.Add(EntityManager->ReserveEntity());
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities.Last(), FTestFragment_Int(i), FTestFragment_Float((float)i));
		}

		AITEST_EQUAL(TEXT("All entities created should be in ArrayArchetype"), EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 0);
		EntityManager->FlushCommands();
		AITEST_EQUAL(TEXT("All entities created should be in ArrayArchetype"), EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), Entities.Num());

		for (int i = 0; i < Entities.Num(); ++i)
		{
			FMassEntityView View(FloatsIntsArchetype, Entities[i]);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Int>().Value, i);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Float>().Value, (float)i);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BuildEntitiesWithFragments, "System.Mass.Commands.BuildEntitiesWithFragments");

struct FCommands_BuildEntitiesInHoles : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 1.25f) * 2; // making sure it's even

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);
		FMath::SRandInit(0);
		Algo::RandomShuffle(Entities);
		EntityManager->BatchDestroyEntities(MakeArrayView(Entities.GetData(), Entities.Num()/2));

		Entities.Reset();
		for (int i = 0; i < EntitiesPerChunk; ++i)
		{
			Entities.Add(EntityManager->ReserveEntity());
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities.Last(), FTestFragment_Int(i), FTestFragment_Float((float)i));
		}

		AITEST_EQUAL(TEXT("All entities created should be in ArrayArchetype"), EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), Count / 2);
		EntityManager->FlushCommands();
		AITEST_EQUAL(TEXT("All entities created should be in ArrayArchetype"), EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), Count / 2 + Entities.Num());

		for (int i = 0; i < Entities.Num(); ++i)
		{
			FMassEntityView View(FloatsIntsArchetype, Entities[i]);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Int>().Value, i);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Float>().Value, (float)i);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BuildEntitiesInHoles, "System.Mass.Commands.BuildEntitiesInHoles");

struct FCommands_BuildEntitiesWithFragmentInstances : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

		TArray<FMassEntityHandle> Entities;
		for (int i = 0; i < Count; ++i)
		{
			Entities.Add(EntityManager->ReserveEntity());
			EntityManager->Defer().PushCommand<FMassCommandBuildEntity>(Entities.Last(), FTestFragment_Int(i), FTestFragment_Float((float)i));
		}

		AITEST_EQUAL(TEXT("All entities created should be in ArrayArchetype"), EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 0);
		EntityManager->FlushCommands();
		AITEST_EQUAL(TEXT("All entities created should be in ArrayArchetype"), EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), Entities.Num());

		for (int i = 0; i < Entities.Num(); ++i)
		{
			FMassEntityView View(FloatsIntsArchetype, Entities[i]);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Int>().Value, i);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Float>().Value, (float)i);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BuildEntitiesWithFragmentInstances, "System.Mass.Commands.BuildEntitiesWithFragmentInstances");

struct FCommands_DeferredFunction : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		const int32 Offset = 1000;

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		int i = 0;
		for (FMassEntityHandle Entity : Entities)
		{
			FMassEntityView View(IntsArchetype, Entity);
			View.GetFragmentData<FTestFragment_Int>().Value = Offset + i++;

			EntityManager->Defer().PushCommand<FMassDeferredSetCommand>([Entity, Archetype = IntsArchetype, Offset](FMassEntityManager&)
				{
					FMassEntityView View(Archetype, Entity);
					View.GetFragmentData<FTestFragment_Int>().Value -= Offset;
				});
		}

		EntityManager->FlushCommands();

		for (i = 0; i < Entities.Num(); ++i)
		{
			FMassEntityView View(IntsArchetype, Entities[i]);
			AITEST_EQUAL(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Int>().Value, i);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_DeferredFunction, "System.Mass.Commands.DeferredFunction");

// pushing commands while the main buffer is being flushed
struct FCommands_PushWhileFlushing : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;

		// here's what we want to do:
		// 1. Create a Count number of Int entities
		// 2. Register TagA observer that will add a float fragment when the tag is added
		//	a. The observer will use EntityManager.Defer() directly for the testing purposes - it should use Context.Defer() in real world scenarios
		// 3. Add TagA to all the created Entities
		// 4. Test if all the affected entities have the float fragment after the flushing

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);
		for (const FMassEntityHandle& EntityHandle : Entities)
		{
			AITEST_NULL(TEXT("None of the freshly created entities is expexted to contain a float fragment")
				, EntityManager->GetFragmentDataPtr<FTestFragment_Float>(EntityHandle));
		}


		UMassTestProcessorBase* ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		ObserverProcessor->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
			{
				for (const FMassEntityHandle& EntityHandle : Context.GetEntities())
				{
					Context.GetEntityManagerChecked().Defer().AddFragment<FTestFragment_Float>(EntityHandle);
				}
			};
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

		EntityManager->Defer().PushCommand<FMassCommandAddTag<FTestTag_A>>(Entities);
		for (const FMassEntityHandle& EntityHandle : Entities)
		{
			AITEST_NULL(TEXT("Pushing the AddTag command should not result in adding the float fragment")
				, EntityManager->GetFragmentDataPtr<FTestFragment_Float>(EntityHandle));
		}

		EntityManager->FlushCommands();
		
		for (const FMassEntityHandle& EntityHandle : Entities)
		{
			AITEST_NOT_NULL(TEXT("After flushing all the observed entities should have the float fragment")
				, EntityManager->GetFragmentDataPtr<FTestFragment_Float>(EntityHandle));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_PushWhileFlushing, "System.Mass.Commands.PushWhileFlushing");

struct FCommands_MoveHandleArrays : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);
		
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchChangeTagsForEntities(EntityCollections, FMassTagBitSet(FTestTag_A::StaticStruct()), FMassTagBitSet());

		// verify that original archetypes no longer host any entities
		AITEST_TRUE("Original archetypes are empty after adding a tag to all entities"
			, EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == 0
				&& EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == 0);

		EntityManager->Defer().PushCommand<FMassCommandRemoveTag<FTestTag_A>>(MoveTemp(Entities));
		EntityManager->FlushCommands();

		AITEST_TRUE("All the entities moved back to the original archetypes"
			, EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == Count
				&& EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype) == Count);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_MoveHandleArrays, "System.Mass.Commands.MoveHandleArrays");

//----------------------------------------------------------------------//
// FMassCommandAddElement / FMassCommandRemoveElement tests
//----------------------------------------------------------------------//
struct FCommands_AddElement_OnlyTags : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_FALSE(TEXT("Entity should not have TagA before the command"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
		}

		FMassCommandAddElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestTag_A::StaticStruct());
		for (const FMassEntityHandle& Entity : Entities)
		{
			Command.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have TagA after flushing"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddElement_OnlyTags, "System.Mass.Commands.AddElement.OnlyTags");

struct FCommands_AddElement_OnlyFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_FALSE(TEXT("Entity should not have FTestFragment_Float before the command"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
		}

		FMassCommandAddElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestFragment_Float::StaticStruct());
		for (const FMassEntityHandle& Entity : Entities)
		{
			Command.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float after flushing"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddElement_OnlyFragments, "System.Mass.Commands.AddElement.OnlyFragments");

struct FCommands_RemoveElement_OnlyTags : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		const FMassArchetypeHandle IntsTagAArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct() });

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsTagAArchetype, Count, Entities);

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have TagA before removal"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
		}

		FMassCommandRemoveElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestTag_A::StaticStruct());
		for (const FMassEntityHandle& Entity : Entities)
		{
			Command.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_FALSE(TEXT("Entity should no longer have TagA after flushing"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElement_OnlyTags, "System.Mass.Commands.RemoveElement.OnlyTags");

struct FCommands_RemoveElement_OnlyFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float before removal"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
		}

		FMassCommandRemoveElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestFragment_Float::StaticStruct());
		for (const FMassEntityHandle& Entity : Entities)
		{
			Command.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_FALSE(TEXT("Entity should no longer have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_EQUAL(TEXT("Entity should have moved to IntsArchetype"), EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElement_OnlyFragments, "System.Mass.Commands.RemoveElement.OnlyFragments");

struct FCommands_RemoveElement_OnlyConstSharedFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add a const shared fragment to all entities
		FTestConstSharedFragment_Int SharedFragmentInstance(42);
		FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(SharedFragmentInstance);
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
		}

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have const shared fragment before removal"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
		}

		FMassCommandRemoveElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestConstSharedFragment_Int::StaticStruct());
		for (const FMassEntityHandle& Entity : Entities)
		{
			Command.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_FALSE(TEXT("Entity should no longer have const shared fragment"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElement_OnlyConstSharedFragments, "System.Mass.Commands.RemoveElement.OnlyConstSharedFragments");

struct FCommands_RemoveElement_MixedTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		const FMassArchetypeHandle FloatsIntsTagAArchetype = EntityManager->CreateArchetype(
			{ FTestFragment_Int::StaticStruct(), FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct() });

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsTagAArchetype, Count, Entities);

		// Add a const shared fragment to all entities
		FTestConstSharedFragment_Int ConstSharedInstance(42);
		FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
		}

		// Add a mutable shared fragment to all entities
		FTestSharedFragment_Int SharedInstance(99);
		FSharedStruct SharedStruct = FSharedStruct::Make(SharedInstance);
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
		SharedFragmentValues.Add(SharedStruct);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchAddSharedFragmentsForEntities(EntityCollections, SharedFragmentValues);

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have all elements before removal"),
				EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct())
				&& EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct())
				&& EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct())
				&& EntityManager->DoesEntityHaveElement(Entity, FTestSharedFragment_Int::StaticStruct()));
		}

		// Push four separate RemoveElement commands for all element types
		FMassCommandRemoveElement& RemoveFragmentCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestFragment_Float::StaticStruct());
		FMassCommandRemoveElement& RemoveTagCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestTag_A::StaticStruct());
		FMassCommandRemoveElement& RemoveConstSharedCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestConstSharedFragment_Int::StaticStruct());
		FMassCommandRemoveElement& RemoveSharedCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestSharedFragment_Int::StaticStruct());
		for (const FMassEntityHandle& Entity : Entities)
		{
			RemoveFragmentCmd.Add(Entity);
			RemoveTagCmd.Add(Entity);
			RemoveConstSharedCmd.Add(Entity);
			RemoveSharedCmd.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have shared"), EntityManager->DoesEntityHaveElement(Entity, FTestSharedFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElement_MixedTypes, "System.Mass.Commands.RemoveElement.MixedTypes");

struct FCommands_AddRemoveElement_MixedTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		const FMassArchetypeHandle IntsTagAArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct() });

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsTagAArchetype, Count, Entities);

		// Add a const shared fragment to all entities
		FTestConstSharedFragment_Int SharedFragmentInstance(42);
		FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(SharedFragmentInstance);
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
		}

		// Add a fragment and a tag via AddElement
		FMassCommandAddElement& AddFragmentCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestFragment_Float::StaticStruct());
		FMassCommandAddElement& AddTagCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestTag_B::StaticStruct());
		// Remove a tag and a const shared fragment via RemoveElement
		FMassCommandRemoveElement& RemoveTagCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestTag_A::StaticStruct());
		FMassCommandRemoveElement& RemoveSharedCmd = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestConstSharedFragment_Int::StaticStruct());

		for (const FMassEntityHandle& Entity : Entities)
		{
			AddFragmentCmd.Add(Entity);
			AddTagCmd.Add(Entity);
			RemoveTagCmd.Add(Entity);
			RemoveSharedCmd.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagB"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_B::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have const shared fragment"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddRemoveElement_MixedTypes, "System.Mass.Commands.AddRemoveElement.MixedTypes");

struct FCommands_RemoveElement_OnlySharedFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add a mutable shared fragment to all entities
		FTestSharedFragment_Int SharedFragmentInstance(42);
		FSharedStruct SharedStruct = FSharedStruct::Make(SharedFragmentInstance);
		FMassArchetypeSharedFragmentValues SharedFragmentValues;
		SharedFragmentValues.Add(SharedStruct);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchAddSharedFragmentsForEntities(EntityCollections, SharedFragmentValues);

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have shared fragment before removal"), EntityManager->DoesEntityHaveElement(Entity, FTestSharedFragment_Int::StaticStruct()));
		}

		FMassCommandRemoveElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(FTestSharedFragment_Int::StaticStruct());
		for (const FMassEntityHandle& Entity : Entities)
		{
			Command.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_FALSE(TEXT("Entity should no longer have shared fragment"), EntityManager->DoesEntityHaveElement(Entity, FTestSharedFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElement_OnlySharedFragments, "System.Mass.Commands.RemoveElement.OnlySharedFragments");

struct FCommands_AddElement_RejectsSharedFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 1;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		FMassCommandAddElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestSharedFragment_Int::StaticStruct());
		Command.Add(Entities[0]);

		{
			MASS_SCOPED_ENSURE_TEST(TEXT("Only tags, fragments, and sparse elements can be added via this API"), Count);
			EntityManager->FlushCommands();
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddElement_RejectsSharedFragment, "System.Mass.Commands.AddElement.RejectsSharedFragment");

struct FCommands_AddElement_RejectsConstSharedFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 1;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		FMassCommandAddElement& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestConstSharedFragment_Int::StaticStruct());
		Command.Add(Entities[0]);

		{
			MASS_SCOPED_ENSURE_TEST(TEXT("Only tags, fragments, and sparse elements can be added via this API"), Count);
			EntityManager->FlushCommands();
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddElement_RejectsConstSharedFragment, "System.Mass.Commands.AddElement.RejectsConstSharedFragment");

struct FCommands_BatchChangeComposition_RemovesSharedFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		const FMassArchetypeHandle IntsTagAArchetype = EntityManager->CreateArchetype(
			{ FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct() });

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsTagAArchetype, Count, Entities);

		// Add a const shared fragment to all entities
		FTestConstSharedFragment_Int ConstSharedInstance(42);
		FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
		}

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have const shared before removal"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagA before removal"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
		}

		// Remove tag + const shared via BatchChangeCompositionForEntities directly
		FMassElementBitSet ElementsToRemove;
		ElementsToRemove.Add(FTestTag_A::StaticStruct());
		ElementsToRemove.Add(FTestConstSharedFragment_Int::StaticStruct());

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchChangeCompositionForEntities(EntityCollections, FMassElementBitSet(), ElementsToRemove);

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchChangeComposition_RemovesSharedFragments, "System.Mass.Commands.BatchChangeComposition.RemovesSharedFragments");

//----------------------------------------------------------------------//
// FMassCommandRemoveElements tests
//----------------------------------------------------------------------//
struct FCommands_RemoveElements_TemplatedMixedTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		const FMassArchetypeHandle IntsFloatsTagAArchetype = EntityManager->CreateArchetype(
			{ FTestFragment_Int::StaticStruct(), FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct() });

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsFloatsTagAArchetype, Count, Entities);

		// Add a const shared fragment
		FTestConstSharedFragment_Int ConstSharedInstance(42);
		FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
		}

		// Add a sparse fragment
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchAddSparseElementToEntities(EntityCollections, FTestFragment_SparseInt::StaticStruct());

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have sparse before removal"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		}

		// Remove fragment + tag + const shared + sparse in one command
		EntityManager->Defer().PushCommand<FMassCommandRemoveElements<FTestFragment_Float, FTestTag_A, FTestConstSharedFragment_Int, FTestFragment_SparseInt>>(Entities);

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have sparse"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElements_TemplatedMixedTypes, "System.Mass.Commands.RemoveElements.TemplatedMixedTypes");

struct FCommands_RemoveElements_RuntimeMixedTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		const FMassArchetypeHandle IntsFloatsTagAArchetype = EntityManager->CreateArchetype(
			{ FTestFragment_Int::StaticStruct(), FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct() });

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsFloatsTagAArchetype, Count, Entities);

		// Add a const shared fragment
		FTestConstSharedFragment_Int ConstSharedInstance(42);
		FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
		}

		// Add a sparse fragment
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchAddSparseElementToEntities(EntityCollections, FTestFragment_SparseInt::StaticStruct());

		// Remove fragment + tag + const shared + sparse via runtime command
		const UScriptStruct* ElementTypes[] = {
			FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct(),
			FTestConstSharedFragment_Int::StaticStruct(), FTestFragment_SparseInt::StaticStruct()
		};
		FMassCommandRemoveElementList& Cmd = EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElementList>(ElementTypes);
		Cmd.Add(Entities);

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have sparse"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElements_RuntimeMixedTypes, "System.Mass.Commands.RemoveElements.RuntimeMixedTypes");

struct FCommands_RemoveElements_ConvenienceMethod : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		const FMassArchetypeHandle IntsFloatsTagAArchetype = EntityManager->CreateArchetype(
			{ FTestFragment_Int::StaticStruct(), FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct() });

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsFloatsTagAArchetype, Count, Entities);

		// Add a const shared fragment
		FTestConstSharedFragment_Int ConstSharedInstance(42);
		FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
		}

		// Add a sparse fragment
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchAddSparseElementToEntities(EntityCollections, FTestFragment_SparseInt::StaticStruct());

		// Use convenience method with all element categories
		EntityManager->Defer().RemoveElements<FTestFragment_Float, FTestTag_A, FTestConstSharedFragment_Int, FTestFragment_SparseInt>(Entities);

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should still have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
			AITEST_FALSE(TEXT("Entity should no longer have sparse"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveElements_ConvenienceMethod, "System.Mass.Commands.RemoveElements.ConvenienceMethod");

//----------------------------------------------------------------------//
// Add command gap tests
//----------------------------------------------------------------------//
struct FCommands_AddElement_RejectsChunkFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 1;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		FMassCommandAddElement& Cmd = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestChunkFragment_Int::StaticStruct());
		Cmd.Add(Entities[0]);

		{
			MASS_SCOPED_ENSURE_TEST(TEXT("chunk fragments require dedicated APIs"), Count);
			EntityManager->FlushCommands();
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddElement_RejectsChunkFragment, "System.Mass.Commands.AddElement.RejectsChunkFragment");

struct FCommands_BatchChangeComposition_RejectsSharedAdd : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 1;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);

		FMassElementBitSet SharedBits;
		SharedBits.Add(FTestConstSharedFragment_Int::StaticStruct());

		{
			MASS_SCOPED_ENSURE_TEST(TEXT("ElementsToAdd contains shared, chunk, or sparse element types"), Count);
			EntityManager->BatchChangeCompositionForEntities(EntityCollections, SharedBits, FMassElementBitSet());
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchChangeComposition_RejectsSharedAdd, "System.Mass.Commands.BatchChangeComposition.RejectsSharedAdd");

struct FCommands_BatchChangeComposition_RejectsSparseAdd : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 1;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);

		FMassElementBitSet SparseBits;
		SparseBits.Add(FTestFragment_SparseInt::StaticStruct());

		{
			MASS_SCOPED_ENSURE_TEST(TEXT("ElementsToAdd contains shared, chunk, or sparse element types"), Count);
			EntityManager->BatchChangeCompositionForEntities(EntityCollections, SparseBits, FMassElementBitSet());
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchChangeComposition_RejectsSparseAdd, "System.Mass.Commands.BatchChangeComposition.RejectsSparseAdd");

struct FCommands_AddElementsWithSharedFragments_AllTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Prepare const shared fragment value
		const FTestConstSharedFragment_Int ConstSharedInstance(42);
		const FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);

		// Add fragment + tag + sparse + const shared value in one command
		for (const FMassEntityHandle& Entity : Entities)
		{
			FMassArchetypeSharedFragmentValues SharedValues;
			SharedValues.Add(ConstSharedStruct);
			EntityManager->Defer().PushCommand<FMassCommandAddElementsWithSharedFragments<FTestFragment_Float, FTestTag_A, FTestFragment_SparseInt>>(
				Entity, MoveTemp(SharedValues));
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have sparse"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddElementsWithSharedFragments_AllTypes, "System.Mass.Commands.AddElementsWithSharedFragments.AllTypes");

struct FCommands_AddElementList_AllTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add fragment + tag + sparse in one runtime command
		const TArray<const UScriptStruct*> ElementTypes = { FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct(), FTestFragment_SparseInt::StaticStruct() };
		FMassCommandAddElementList& Command = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElementList>(ElementTypes);
		for (const FMassEntityHandle& Entity : Entities)
		{
			Command.Add(Entity);
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Int (original)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagA (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have sparse (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddElementList_AllTypes, "System.Mass.Commands.AddElementList.AllTypes");

struct FCommands_AddFragmentInstances_AllTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add fragment with value + tag + sparse fragment + sparse tag in a single command
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float, FTestTag_A, FTestFragment_SparseInt, FTestTag_SparseA>>(
				Entity, FTestFragment_Float(2.5f), FTestTag_A(), FTestFragment_SparseInt(77), FTestTag_SparseA());
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Int (original)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float (added with value)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagA (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseInt (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseTagA (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

			const FMassEntityView EntityView(*EntityManager, Entity);
			AITEST_EQUAL(TEXT("Float fragment value should be 2.5f"), EntityView.GetFragmentData<FTestFragment_Float>().Value, 2.5f);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddFragmentInstances_AllTypes, "System.Mass.Commands.AddFragmentInstances.AllTypes");

struct FCommands_BuildEntity_AllTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		Entities.Reserve(Count);
		for (int32 i = 0; i < Count; ++i)
		{
			Entities.Add(EntityManager->ReserveEntity());
		}

		// Build entity with fragment values + tag + sparse fragment + sparse tag
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->Defer().PushCommand<FMassCommandBuildEntity<FTestFragment_Int, FTestFragment_Float, FTestTag_A, FTestFragment_SparseInt, FTestTag_SparseA>>(
				Entity, FTestFragment_Int(10), FTestFragment_Float(1.5f), FTestTag_A(), FTestFragment_SparseInt(88), FTestTag_SparseA());
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should be valid"), EntityManager->IsEntityValid(Entity));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseInt"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseTagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

			const FMassEntityView EntityView(*EntityManager, Entity);
			AITEST_EQUAL(TEXT("Int value should be 10"), EntityView.GetFragmentData<FTestFragment_Int>().Value, 10);
			AITEST_EQUAL(TEXT("Float value should be 1.5f"), EntityView.GetFragmentData<FTestFragment_Float>().Value, 1.5f);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BuildEntity_AllTypes, "System.Mass.Commands.BuildEntity.AllTypes");

struct FCommands_AddFragmentInstancesWithSharedFragments_AllTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		const FTestConstSharedFragment_Int ConstSharedInstance(42);
		const FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);

		// Add fragment with value + tag + sparse fragment + sparse tag + shared value in one command
		for (const FMassEntityHandle& Entity : Entities)
		{
			FMassArchetypeSharedFragmentValues SharedValues;
			SharedValues.Add(ConstSharedStruct);
			EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstancesWithSharedFragments<FMassArchetypeSharedFragmentValues, FTestFragment_Float, FTestTag_A, FTestFragment_SparseInt, FTestTag_SparseA>>(
				Entity, MoveTemp(SharedValues), FTestFragment_Float(3.14f), FTestTag_A(), FTestFragment_SparseInt(55), FTestTag_SparseA());
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Int (original)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Float (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagA (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseInt (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseTagA (added)"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));

			const FMassEntityView EntityView(*EntityManager, Entity);
			AITEST_EQUAL(TEXT("Float value should be 3.14f"), EntityView.GetFragmentData<FTestFragment_Float>().Value, 3.14f);
			AITEST_EQUAL(TEXT("Const shared value should be 42"), EntityView.GetConstSharedFragmentData<FTestConstSharedFragment_Int>().Value, 42);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddFragmentInstancesWithSharedFragments_AllTypes, "System.Mass.Commands.AddFragmentInstancesWithSharedFragments.AllTypes");

struct FCommands_BuildEntityWithSharedFragments_AllTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		Entities.Reserve(Count);
		for (int32 i = 0; i < Count; ++i)
		{
			Entities.Add(EntityManager->ReserveEntity());
		}

		const FTestConstSharedFragment_Int ConstSharedInstance(99);
		const FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);

		// Build entity with fragment values + tag + sparse fragment + sparse tag + shared value
		for (const FMassEntityHandle& Entity : Entities)
		{
			FMassArchetypeSharedFragmentValues SharedValues;
			SharedValues.Add(ConstSharedStruct);
			EntityManager->Defer().PushCommand<FMassCommandBuildEntityWithSharedFragments<FMassArchetypeSharedFragmentValues, FTestFragment_Int, FTestTag_A, FTestFragment_SparseInt, FTestTag_SparseA>>(
				Entity, MoveTemp(SharedValues), FTestFragment_Int(7), FTestTag_A(), FTestFragment_SparseInt(33), FTestTag_SparseA());
		}

		EntityManager->FlushCommands();

		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE(TEXT("Entity should be valid"), EntityManager->IsEntityValid(Entity));
			AITEST_TRUE(TEXT("Entity should have FTestFragment_Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseInt"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have SparseTagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
			AITEST_TRUE(TEXT("Entity should have const shared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));

			const FMassEntityView EntityView(*EntityManager, Entity);
			AITEST_EQUAL(TEXT("Int value should be 7"), EntityView.GetFragmentData<FTestFragment_Int>().Value, 7);
			AITEST_EQUAL(TEXT("Const shared value should be 99"), EntityView.GetConstSharedFragmentData<FTestConstSharedFragment_Int>().Value, 99);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BuildEntityWithSharedFragments_AllTypes, "System.Mass.Commands.BuildEntityWithSharedFragments.AllTypes");

//----------------------------------------------------------------------//
// Benchmarks: Separate vs Mixed-Element Commands
// Each iteration runs the Mixed arm first, then the Separate arm. Mixed therefore
// always operates on a colder cache; Separate benefits from archetype tables and
// allocator pools already warmed by the Mixed run. Results reflect this ordering bias.
//----------------------------------------------------------------------//
DEFINE_LOG_CATEGORY_STATIC(LogMassCommandsBench, Log, All);

namespace
{
	constexpr int32 BenchEntityCount = 10000;
	constexpr int32 BenchIterations = 10;

	double TimedFlush(const TSharedPtr<FMassEntityManager>& EntityManager)
	{
		const double Start = FPlatformTime::Seconds();
		EntityManager->FlushCommands();
		const double End = FPlatformTime::Seconds();
		return (End - Start) * 1000.0;
	}
}

struct FBench_AddElements_SeparateVsMixed : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		double MixedTotal = 0.0;
		double SeparateTotal = 0.0;

		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Mixed
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestFragment_Float, FTestTag_A, FTestFragment_SparseInt>>(Entity);
				}
				MixedTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Separate
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsArchetype, BenchEntityCount, Entities);
				FMassCommandAddElement& CmdFloat = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestFragment_Float::StaticStruct());
				FMassCommandAddElement& CmdTag = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestTag_A::StaticStruct());
				FMassCommandAddElement& CmdSparse = EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(FTestFragment_SparseInt::StaticStruct());
				for (const FMassEntityHandle& Entity : Entities)
				{
					CmdFloat.Add(Entity);
					CmdTag.Add(Entity);
					CmdSparse.Add(Entity);
				}
				SeparateTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SeparateAvg = SeparateTotal / BenchIterations;
		const double MixedAvg = MixedTotal / BenchIterations;
		const double DeltaPct = SeparateAvg > 0.0 ? ((MixedAvg - SeparateAvg) / SeparateAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("AddElements: separate %.3f ms, mixed %.3f ms, delta: %+.1f%%"), SeparateAvg, MixedAvg, DeltaPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, 5, Entities);
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestFragment_Float, FTestTag_A, FTestFragment_SparseInt>>(Entity);
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_TRUE(TEXT("Should have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_TRUE(TEXT("Should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
				AITEST_TRUE(TEXT("Should have Sparse"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_AddElements_SeparateVsMixed, "System.Mass.Commands.Benchmark.AddElements_SeparateVsMixed");

struct FBench_AddFragmentInstances_SeparateVsMixed : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		double MixedTotal = 0.0;
		double SeparateTotal = 0.0;

		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Mixed
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float, FTestTag_A>>(Entity, FTestFragment_Float(3.14f), FTestTag_A());
				}
				MixedTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Separate
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float>>(Entity, FTestFragment_Float(3.14f));
					EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestTag_A>>(Entity);
				}
				SeparateTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SeparateAvg = SeparateTotal / BenchIterations;
		const double MixedAvg = MixedTotal / BenchIterations;
		const double DeltaPct = SeparateAvg > 0.0 ? ((MixedAvg - SeparateAvg) / SeparateAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("AddFragmentInstances: separate %.3f ms, mixed %.3f ms, delta: %+.1f%%"), SeparateAvg, MixedAvg, DeltaPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, 5, Entities);
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float, FTestTag_A>>(Entity, FTestFragment_Float(3.14f), FTestTag_A());
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_TRUE(TEXT("Should have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_TRUE(TEXT("Should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
				const FMassEntityView View(*EntityManager, Entity);
				AITEST_EQUAL(TEXT("Float value"), View.GetFragmentData<FTestFragment_Float>().Value, 3.14f);
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_AddFragmentInstances_SeparateVsMixed, "System.Mass.Commands.Benchmark.AddFragmentInstances_SeparateVsMixed");

struct FBench_AddFragmentInstancesWithShared_DeferredVsMixed : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FTestConstSharedFragment_Int ConstSharedInstance(42);
		const FConstSharedStruct ConstSharedStruct = EntityManager->GetOrCreateConstSharedFragment(ConstSharedInstance);

		double MixedTotal = 0.0;
		double SeparateTotal = 0.0;

		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Mixed
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					FMassArchetypeSharedFragmentValues SharedValues;
					SharedValues.Add(ConstSharedStruct);
					EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstancesWithSharedFragments<FMassArchetypeSharedFragmentValues, FTestFragment_Float>>(
						Entity, MoveTemp(SharedValues), FTestFragment_Float(3.14f));
				}
				MixedTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Separate
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float>>(Entity, FTestFragment_Float(3.14f));
					EntityManager->Defer().PushCommand<FMassDeferredAddCommand>([Entity, ConstSharedStruct](FMassEntityManager& InEntityManager)
						{
							if (InEntityManager.IsEntityValid(Entity))
							{
								InEntityManager.AddConstSharedFragmentToEntity(Entity, ConstSharedStruct);
							}
						});
				}
				SeparateTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SeparateAvg = SeparateTotal / BenchIterations;
		const double MixedAvg = MixedTotal / BenchIterations;
		const double DeltaPct = SeparateAvg > 0.0 ? ((MixedAvg - SeparateAvg) / SeparateAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("AddFragmentInstances+Shared: separate %.3f ms, mixed %.3f ms, delta: %+.1f%%"), SeparateAvg, MixedAvg, DeltaPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, 5, Entities);
			for (const FMassEntityHandle& Entity : Entities)
			{
				FMassArchetypeSharedFragmentValues SharedValues;
				SharedValues.Add(ConstSharedStruct);
				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstancesWithSharedFragments<FMassArchetypeSharedFragmentValues, FTestFragment_Float>>(
					Entity, MoveTemp(SharedValues), FTestFragment_Float(3.14f));
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_TRUE(TEXT("Should have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_TRUE(TEXT("Should have ConstShared"), EntityManager->DoesEntityHaveElement(Entity, FTestConstSharedFragment_Int::StaticStruct()));
				const FMassEntityView View(*EntityManager, Entity);
				AITEST_EQUAL(TEXT("Float value"), View.GetFragmentData<FTestFragment_Float>().Value, 3.14f);
				AITEST_EQUAL(TEXT("Shared value"), View.GetConstSharedFragmentData<FTestConstSharedFragment_Int>().Value, 42);
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_AddFragmentInstancesWithShared_DeferredVsMixed, "System.Mass.Commands.Benchmark.AddFragmentInstancesWithShared_DeferredVsMixed");

struct FBench_BuildEntity_SeparateVsMixed : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		double MixedTotal = 0.0;
		double SeparateTotal = 0.0;

		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Mixed
			{
				TArray<FMassEntityHandle> Entities;
				Entities.Reserve(BenchEntityCount);
				for (int32 i = 0; i < BenchEntityCount; ++i) { Entities.Add(EntityManager->ReserveEntity()); }
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandBuildEntity<FTestFragment_Int, FTestFragment_Float, FTestTag_A>>(
						Entity, FTestFragment_Int(10), FTestFragment_Float(2.5f), FTestTag_A());
				}
				MixedTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Separate
			{
				TArray<FMassEntityHandle> Entities;
				Entities.Reserve(BenchEntityCount);
				for (int32 i = 0; i < BenchEntityCount; ++i) { Entities.Add(EntityManager->ReserveEntity()); }
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandBuildEntity<FTestFragment_Int, FTestFragment_Float>>(
						Entity, FTestFragment_Int(10), FTestFragment_Float(2.5f));
					EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestTag_A>>(Entity);
				}
				SeparateTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SeparateAvg = SeparateTotal / BenchIterations;
		const double MixedAvg = MixedTotal / BenchIterations;
		const double DeltaPct = SeparateAvg > 0.0 ? ((MixedAvg - SeparateAvg) / SeparateAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("BuildEntity: separate %.3f ms, mixed %.3f ms, delta: %+.1f%%"), SeparateAvg, MixedAvg, DeltaPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			Entities.Reserve(5);
			for (int32 i = 0; i < 5; ++i) { Entities.Add(EntityManager->ReserveEntity()); }
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandBuildEntity<FTestFragment_Int, FTestFragment_Float, FTestTag_A>>(
					Entity, FTestFragment_Int(10), FTestFragment_Float(2.5f), FTestTag_A());
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_TRUE(TEXT("Should have Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
				AITEST_TRUE(TEXT("Should have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_TRUE(TEXT("Should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
				const FMassEntityView View(*EntityManager, Entity);
				AITEST_EQUAL(TEXT("Int value"), View.GetFragmentData<FTestFragment_Int>().Value, 10);
				AITEST_EQUAL(TEXT("Float value"), View.GetFragmentData<FTestFragment_Float>().Value, 2.5f);
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_BuildEntity_SeparateVsMixed, "System.Mass.Commands.Benchmark.BuildEntity_SeparateVsMixed");

//----------------------------------------------------------------------//
// Parity benchmarks: single-type vs multi-type command overhead
// Measures whether multi-type command instantiations carry dispatch overhead relative
// to single-type instantiations. Both arms must perform identical composition transitions;
// where possible the extra type in the multi arm is already present in the source archetype
// so it is a no-op at the archetype level. BuildEntity is an exception: reserved entities
// have no existing composition, so the arms build genuinely different archetypes.
//----------------------------------------------------------------------//

struct FBench_AddElements_SingleVsMultiType : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		double MultiTotal = 0.0;
		double SingleTotal = 0.0;

		// FTestTag_A is already in IntsTagAArchetype, so the multi arm's extra type is a
		// no-op at the composition level. Both arms perform the identical transition
		// ({Int,Tag_A} -> {Int,Tag_A,Float}); the only variable is command dispatch overhead.
		const FMassArchetypeHandle IntsTagAArchetype = EntityManager->CreateArchetype(
			{FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct()});

		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Multi-type: FTestTag_A already present — no-op for Tag_A at composition level
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsTagAArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestFragment_Float, FTestTag_A>>(Entity);
				}
				MultiTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Single-type
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsTagAArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestFragment_Float>>(Entity);
				}
				SingleTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SingleAvg = SingleTotal / BenchIterations;
		const double MultiAvg = MultiTotal / BenchIterations;
		const double OverheadPct = SingleAvg > 0.0 ? ((MultiAvg - SingleAvg) / SingleAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("AddElements parity: single-type %.3f ms, multi-type %.3f ms, overhead: %+.1f%%"), SingleAvg, MultiAvg, OverheadPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, 5, Entities);
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestFragment_Float, FTestTag_A>>(Entity);
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_TRUE(TEXT("Should have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_TRUE(TEXT("Should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_AddElements_SingleVsMultiType, "System.Mass.Commands.Benchmark.AddElements_SingleVsMultiType");

struct FBench_RemoveElements_SingleVsMultiType : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		double MultiTotal = 0.0;
		double SingleTotal = 0.0;

		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Multi-type: FTestTag_A is not present in FloatsIntsArchetype, so the removal is a no-op for that type.
			// Both arms therefore perform the identical archetype transition ({Float,Int} -> {Int}); the only
			// variable is single-type vs multi-type command dispatch overhead.
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(FloatsIntsArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandRemoveElements<FTestFragment_Float, FTestTag_A>>(Entity);
				}
				MultiTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Single-type
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(FloatsIntsArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandRemoveElements<FTestFragment_Float>>(Entity);
				}
				SingleTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SingleAvg = SingleTotal / BenchIterations;
		const double MultiAvg = MultiTotal / BenchIterations;
		const double OverheadPct = SingleAvg > 0.0 ? ((MultiAvg - SingleAvg) / SingleAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("RemoveElements parity: single-type %.3f ms, multi-type %.3f ms, overhead: %+.1f%%"), SingleAvg, MultiAvg, OverheadPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(FloatsIntsArchetype, 5, Entities);
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandAddElements<FTestTag_A>>(Entity);
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandRemoveElements<FTestFragment_Float, FTestTag_A>>(Entity);
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_FALSE(TEXT("Should not have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_FALSE(TEXT("Should not have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
				AITEST_TRUE(TEXT("Should still have Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_RemoveElements_SingleVsMultiType, "System.Mass.Commands.Benchmark.RemoveElements_SingleVsMultiType");

struct FBench_AddFragmentInstances_SingleVsMultiType : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		double MultiTotal = 0.0;
		double SingleTotal = 0.0;

		// FTestTag_A is already in IntsTagAArchetype, so the multi arm's extra type is a
		// no-op at the composition level. Tags also carry no value payload (GetAsNonTagGenericMultiArray
		// excludes them), so the command buffers are equivalent in data volume. Both arms perform
		// the identical transition ({Int,Tag_A} -> {Int,Tag_A,Float}); the only variable is
		// command dispatch overhead.
		const FMassArchetypeHandle IntsTagAArchetype = EntityManager->CreateArchetype(
			{FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct()});

		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Multi-type: FTestTag_A already present — no-op for Tag_A at composition level;
			// tag value excluded from payload by GetAsNonTagGenericMultiArray
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsTagAArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float, FTestTag_A>>(Entity, FTestFragment_Float(3.14f), FTestTag_A());
				}
				MultiTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Single-type
			{
				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchCreateEntities(IntsTagAArchetype, BenchEntityCount, Entities);
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float>>(Entity, FTestFragment_Float(3.14f));
				}
				SingleTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SingleAvg = SingleTotal / BenchIterations;
		const double MultiAvg = MultiTotal / BenchIterations;
		const double OverheadPct = SingleAvg > 0.0 ? ((MultiAvg - SingleAvg) / SingleAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("AddFragmentInstances parity: single-type %.3f ms, multi-type %.3f ms, overhead: %+.1f%%"), SingleAvg, MultiAvg, OverheadPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, 5, Entities);
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances<FTestFragment_Float, FTestTag_A>>(Entity, FTestFragment_Float(3.14f), FTestTag_A());
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_TRUE(TEXT("Should have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_TRUE(TEXT("Should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
				const FMassEntityView View(*EntityManager, Entity);
				AITEST_EQUAL(TEXT("Float value"), View.GetFragmentData<FTestFragment_Float>().Value, 3.14f);
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_AddFragmentInstances_SingleVsMultiType, "System.Mass.Commands.Benchmark.AddFragmentInstances_SingleVsMultiType");

struct FBench_BuildEntity_SingleVsMultiType : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		double MultiTotal = 0.0;
		double SingleTotal = 0.0;

		// Note: reserved entities have no existing composition, so there is no "already-present type"
		// trick available. The multi arm builds {Int,Float,Tag_A} and the single arm builds {Int,Float};
		// the delta includes real composition cost (extra archetype lookup + Tag_A in the type set),
		// not pure dispatch overhead. After the first iteration both archetypes are cached so the
		// archetype-creation cost is amortized, but the arms remain methodologically asymmetric.
		for (int32 Iter = 0; Iter < BenchIterations; ++Iter)
		{
			// Multi-type
			{
				TArray<FMassEntityHandle> Entities;
				Entities.Reserve(BenchEntityCount);
				for (int32 i = 0; i < BenchEntityCount; ++i) { Entities.Add(EntityManager->ReserveEntity()); }
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandBuildEntity<FTestFragment_Int, FTestFragment_Float, FTestTag_A>>(
						Entity, FTestFragment_Int(10), FTestFragment_Float(2.5f), FTestTag_A());
				}
				MultiTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
			// Single-type
			{
				TArray<FMassEntityHandle> Entities;
				Entities.Reserve(BenchEntityCount);
				for (int32 i = 0; i < BenchEntityCount; ++i) { Entities.Add(EntityManager->ReserveEntity()); }
				for (const FMassEntityHandle& Entity : Entities)
				{
					EntityManager->Defer().PushCommand<FMassCommandBuildEntity<FTestFragment_Int, FTestFragment_Float>>(
						Entity, FTestFragment_Int(10), FTestFragment_Float(2.5f));
				}
				SingleTotal += TimedFlush(EntityManager);
				EntityManager->BatchDestroyEntities(Entities);
			}
		}

		const double SingleAvg = SingleTotal / BenchIterations;
		const double MultiAvg = MultiTotal / BenchIterations;
		const double OverheadPct = SingleAvg > 0.0 ? ((MultiAvg - SingleAvg) / SingleAvg) * 100.0 : 0.0;
		UE_LOG(LogMassCommandsBench, Log, TEXT("BuildEntity parity: single-type %.3f ms, multi-type %.3f ms, overhead: %+.1f%%"), SingleAvg, MultiAvg, OverheadPct);

		// Validate once
		{
			TArray<FMassEntityHandle> Entities;
			Entities.Reserve(5);
			for (int32 i = 0; i < 5; ++i) { Entities.Add(EntityManager->ReserveEntity()); }
			for (const FMassEntityHandle& Entity : Entities)
			{
				EntityManager->Defer().PushCommand<FMassCommandBuildEntity<FTestFragment_Int, FTestFragment_Float, FTestTag_A>>(
					Entity, FTestFragment_Int(10), FTestFragment_Float(2.5f), FTestTag_A());
			}
			EntityManager->FlushCommands();
			for (const FMassEntityHandle& Entity : Entities)
			{
				AITEST_TRUE(TEXT("Should have Int"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
				AITEST_TRUE(TEXT("Should have Float"), EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
				AITEST_TRUE(TEXT("Should have TagA"), EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
				const FMassEntityView View(*EntityManager, Entity);
				AITEST_EQUAL(TEXT("Int value"), View.GetFragmentData<FTestFragment_Int>().Value, 10);
				AITEST_EQUAL(TEXT("Float value"), View.GetFragmentData<FTestFragment_Float>().Value, 2.5f);
			}
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBench_BuildEntity_SingleVsMultiType, "System.Mass.Commands.Benchmark.BuildEntity_SingleVsMultiType");

#endif // WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// Compile-time validation tests for typed bitsets (DECLARE_NEWTYPEBITSET).
// Uncomment any line below to verify it produces a static_assert error:
//   "Type does not match the expected element type for this bitset."
//----------------------------------------------------------------------//
#if 0
static void BitsetTypeValidation_ShouldNotCompile()
{
	// Tag in a fragment bitset
	FMassFragmentBitSet FragBitSet;
	FragBitSet += FMassFragmentBitSet::GetTypeBitSet<FTestTag_A>();

	// Const shared fragment in a fragment bitset
	FragBitSet += FMassFragmentBitSet::GetTypeBitSet<FTestConstSharedFragment_Int>();

	// Fragment in a tag bitset
	FMassTagBitSet TagBitSet;
	TagBitSet += FMassTagBitSet::GetTypeBitSet<FTestFragment_Int>();

	// PopulateBitSet path (what commands use internally)
	FMassFragmentBitSet Result;
	UE::Mass::TMultiTypeList<FTestConstSharedFragment_Int>::PopulateBitSet(Result);

	// ConstructFragmentBitSet helper
	auto BitSet = UE::Mass::Utils::ConstructFragmentBitSet<EMassCommandCheckTime::CompileTimeCheck, FTestTag_A>();
}
#endif

} // FMassCommandsTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
