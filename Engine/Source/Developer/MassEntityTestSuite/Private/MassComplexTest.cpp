// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassExecutionContext.h"
#include "MassProcessingPhaseManager.h"
#include "MassCommands.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

DEFINE_LOG_CATEGORY_STATIC(LogMassComplexTest, Log, All);

//-----------------------------------------------------------------------------
// This file contains tests spanning multiple functionalities.
//-----------------------------------------------------------------------------
namespace FMassComplexTest
{
//-----------------------------------------------------------------------------
// Testing the issue first reported here https://forums.unrealengine.com/t/masscommandaddfragmentinstances-adding-fragments-to-incorrect-entities/731341
// 
// The issue boiled down to FMassArchetypeData::BatchMoveEntitiesToAnotherArchetype (that's the function that does the 
// work behind FMassCommandAddFragmentInstances used in this test) was moving entities to the new archetype in a changed 
// order, while the call site (FMassEntityManager::BatchAddFragmentInstancesForEntities) the order has not changed and 
// subsequently assigned mismatched values to the freshly moved entities (via FMassArchetypeData::BatchSetFragmentValues)
// 
// The fix was as simple as splitting BatchMoveEntitiesToAnotherArchetype into two sections: 
// 1. adding all the new entities to the new archetype (that we do in original order) 
// 2. and only then removal from the previous archetype (that we need to do back-to-front to keep the chunk and entity indices relevant). 
// This way the order of entities moved to the new archetype remains the same as in the input data.
//-----------------------------------------------------------------------------
struct FComplex_AddingFragmentInstancesToDiscontinouousEntitiesCollection : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int EntitiesCount = 4;
		constexpr int MiddleIndex = EntitiesCount / 2;
		CA_ASSUME(EntityManager);

		// The setup of the test.
		// We create EntitiesCount entities containing only a FTestFragment_Int fragment and then set the value
		// to subsequent integer values
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(IntsArchetype, EntitiesCount, EntitiesCreated);

		FMassExecutionContext ExecContext(*EntityManager.Get());
		FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Int::StaticStruct() });
		Query.ForEachEntityChunk(ExecContext, [](FMassExecutionContext& Context)
			{
				TArrayView<FTestFragment_Int> Ints = Context.GetMutableFragmentView<FTestFragment_Int>();
				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					Ints[i].Value = i;
				}
			});

		for (int i = EntitiesCount - 1; i >= 0; --i)
		{
			// skipping one index somewhere in the middle to ensure discontinuity in entities being processed by the command
			if (i != MiddleIndex)
			{
				const FTestFragment_Int& IntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(EntitiesCreated[i]);
				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(EntitiesCreated[i], FTestFragment_Float(static_cast<float>(IntFragment.Value)));
			}
		}
		EntityManager->FlushCommands();

		// by now we should have 1 remaining entity in the original IntsArchetype (no point in testing it, there are 
		// other tests doing that), and 3 entities in the Ints&Floats archetype, with value of Int fragment and Float 
		// fragment matching.
		for (int i = 0; i < EntitiesCount; ++i)
		{
			if (i != MiddleIndex)
			{
				const FTestFragment_Float& FloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(EntitiesCreated[i]);
				const FTestFragment_Int& IntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(EntitiesCreated[i]);
				
				AITEST_EQUAL("All int fragments are expected to contain the value indicating the order in which the entity has been created.", IntFragment.Value, i);
				AITEST_EQUAL("The int and float values are expected to remain in sync.", IntFragment.Value, (int)FloatFragment.Value);
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FComplex_AddingFragmentInstancesToDiscontinouousEntitiesCollection, "System.Mass.Complex.AddingFragmentInstancesToDiscontinouousEntitiesCollection");

//-----------------------------------------------------------------------------
// Entity creation in scheduled processors
//-----------------------------------------------------------------------------
struct FComplex_CreateEntitiesParallel : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;

	static constexpr int32 NumToSpawn = 1000000;
	static constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	TObjectPtr<UMassTestProcessorBase> CreateIntsProcessor = nullptr;
	TObjectPtr<UMassTestProcessorBase> CreateFloatsProcessor = nullptr;
	TObjectPtr<UMassTestProcessorBase> CreateLargeProcessor = nullptr;

	FMassElementBitSet IntsComposition;
	FMassElementBitSet FloatsComposition;
	FMassElementBitSet LargeComposition;

	virtual bool PopulatePhasesConfig() override
	{
		return true;
	}

	virtual bool SetUp() override
	{
		if (!Super::SetUp())
		{
			return false;
		}

		IntsComposition = IntsArchetype.GetCompositionBitSetChecked();
		CreateIntsProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateIntsProcessor->SetShouldAllowMultipleInstances(true);
		CreateIntsProcessor->SetProcessingPhase(PhaseToRun);
		CreateIntsProcessor->GetEntityCreationRequirements().AddCreatedArchetype(IntsComposition);
		CreateIntsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Creating IntsArchetype");

				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchReserveEntities(NumToSpawn, Entities);
				for (int32 i = 0; i < NumToSpawn; ++i)
				{
					Context.BuildEntity(IntsArchetype, Entities[i]);
					FMassEntityView EntityView(ExecuteEntityManager, Entities[i]);
					EntityView.GetFragmentData<FTestFragment_Int>().Value = 1;
				}

				UE_LOGF(LogMassComplexTest, Log, "End Creating IntsArchetype");
			};
		PhaseManager->RegisterDynamicProcessor(*CreateIntsProcessor);

		FloatsComposition = FloatsArchetype.GetCompositionBitSetChecked();
		CreateFloatsProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateFloatsProcessor->SetShouldAllowMultipleInstances(true);
		CreateFloatsProcessor->SetProcessingPhase(PhaseToRun);
		CreateFloatsProcessor->GetEntityCreationRequirements().AddCreatedArchetype(FloatsComposition);
		CreateFloatsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Creating FloatsArchetype");

				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchReserveEntities(NumToSpawn, Entities);
				for (int32 i = 0; i < NumToSpawn; ++i)
				{
					Context.BuildEntity(FloatsArchetype, Entities[i]);
					FMassEntityView EntityView(ExecuteEntityManager, Entities[i]);
					EntityView.GetFragmentData<FTestFragment_Float>().Value = 1.0f;
				}

				UE_LOGF(LogMassComplexTest, Log, "End Creating FloatsArchetype");
			};
		PhaseManager->RegisterDynamicProcessor(*CreateFloatsProcessor);

		LargeComposition = LargeArchetype.GetCompositionBitSetChecked();
		CreateLargeProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateLargeProcessor->SetShouldAllowMultipleInstances(true);
		CreateLargeProcessor->SetProcessingPhase(PhaseToRun);
		CreateLargeProcessor->GetEntityCreationRequirements().AddCreatedArchetype(LargeComposition);
		CreateLargeProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Creating LargeArchetype"); 

				TArray<FMassEntityHandle> Entities;
				EntityManager->BatchReserveEntities(NumToSpawn, Entities);
				for (int32 i = 0; i < NumToSpawn; ++i)
				{
					Context.BuildEntity(LargeArchetype, Entities[i]);
				}

				UE_LOGF(LogMassComplexTest, Log, "End Creating LargeArchetype");
			};
		PhaseManager->RegisterDynamicProcessor(*CreateLargeProcessor);


		return true;
	}

	virtual bool Update() override
	{
		Super::Update();
		return true;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();

#if WITH_MASSENTITY_DEBUG

		// Verify that entities ended up in IntsArchetype.
		const int32 IntsCount = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in IntsArchetype"), IntsCount, NumToSpawn);

		const int32 FloatsCount = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in FloatsArchetype"), FloatsCount, NumToSpawn);

		const int32 LargeCount = EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in LargeArchetype"), LargeCount, NumToSpawn);

#endif //WITH_MASSENTITY_DEBUG

		PhaseManager->UnregisterDynamicProcessor(*CreateIntsProcessor);
		CreateIntsProcessor->MarkAsGarbage();
		CreateIntsProcessor = nullptr;
		PhaseManager->UnregisterDynamicProcessor(*CreateFloatsProcessor);
		CreateFloatsProcessor->MarkAsGarbage();
		CreateFloatsProcessor = nullptr;
		PhaseManager->UnregisterDynamicProcessor(*CreateLargeProcessor);
		CreateLargeProcessor->MarkAsGarbage();
		CreateLargeProcessor = nullptr;
	}
};
IMPLEMENT_AI_LATENT_TEST(FComplex_CreateEntitiesParallel, "System.Mass.Complex.CreateEntitiesParallel");

struct FComplex_CreateEntitiesParallelWithProcessingQueue : public FComplex_CreateEntitiesParallel
{
	virtual bool SetUp() override
	{
		if (!FComplex_CreateEntitiesParallel::SetUp())
		{
			return false;
		}
		return true;
	}
};
IMPLEMENT_AI_LATENT_TEST(FComplex_CreateEntitiesParallelWithProcessingQueue, "System.Mass.Complex.CreateEntitiesParallelWithProcessingQueue");

//-----------------------------------------------------------------------------
// Create Entities during query iteration
//-----------------------------------------------------------------------------
struct FComplex_CreateEntitiesDuringQuery : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;

	static constexpr int32 NumToSpawn = 10;
	static constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	TObjectPtr<UMassTestProcessorBase> CreateEntitiesProcessor = nullptr;
	TObjectPtr<UMassTestProcessorBase> TestObserver = nullptr;
	int32 NumEntitiesObserved = 0;

	FMassElementBitSet FloatsComposition;
	FMassElementBitSet LargeComposition;
	virtual bool PopulatePhasesConfig() override
	{
		return true;
	}

	virtual bool SetUp() override
	{
		if (!Super::SetUp())
		{
			return false;
		}

		TestObserver = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		TestObserver->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::CreateEntity, TestObserver);
		TestObserver->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Observing Created Entities");

				TestObserver->EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
					{
						for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
						{
							NumEntitiesObserved++;
						}
					});

				UE_LOGF(LogMassComplexTest, Log, "End Observing Created Entities");
			};

		TArray<FMassEntityHandle> Created;
		Created.Reserve(NumToSpawn);
		EntityManager->BatchCreateEntities(IntsArchetype, NumToSpawn, Created);

		FloatsComposition = FloatsArchetype.GetCompositionBitSetChecked();
		LargeComposition = LargeArchetype.GetCompositionBitSetChecked();

		CreateEntitiesProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateEntitiesProcessor->SetShouldAllowMultipleInstances(true);
		CreateEntitiesProcessor->SetProcessingPhase(PhaseToRun);
		CreateEntitiesProcessor->CallInitialize(GetTransientPackage(), EntityManager->AsShared());
		CreateEntitiesProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		CreateEntitiesProcessor->EntityQuery.AddCreatedArchetype(FloatsComposition);
		CreateEntitiesProcessor->EntityQuery.AddCreatedArchetype(LargeComposition);

		CreateEntitiesProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Creating Entities During Query");

				CreateEntitiesProcessor->EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
					{
						for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
						{
							Context.CreateEntity(FloatsArchetype);
							Context.CreateEntity(LargeArchetype);
						}
					});

				UE_LOGF(LogMassComplexTest, Log, "End Creating Entities During Query");
			};
		PhaseManager->RegisterDynamicProcessor(*CreateEntitiesProcessor);

		return true;
	}

	virtual bool Update() override
	{
		Super::Update();
		return true;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();

#if WITH_MASSENTITY_DEBUG
		// Verify that entities ended up in IntsArchetype.
		const int32 IntsCount = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in IntsArchetype"), IntsCount, NumToSpawn);

		const int32 FloatsCount = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in FloatsArchetype"), FloatsCount, NumToSpawn);
		AITEST_EQUAL_LATENT(TEXT("Expected number of observed entities to match number created"), FloatsCount, NumEntitiesObserved);

		const int32 LargeCount = EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in LargeArchetype"), LargeCount, NumToSpawn);
#endif //WITH_MASSENTITY_DEBUG

		PhaseManager->UnregisterDynamicProcessor(*CreateEntitiesProcessor);
		CreateEntitiesProcessor->MarkAsGarbage();
		CreateEntitiesProcessor = nullptr;
	}
};
IMPLEMENT_AI_LATENT_TEST(FComplex_CreateEntitiesDuringQuery, "System.Mass.Complex.CreateEntitiesDuringQuery");

//-----------------------------------------------------------------------------
// Deferred Create Entities during query iteration
//-----------------------------------------------------------------------------
struct FComplex_DeferredCreateEntitiesDuringQuery : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;

	static constexpr int32 NumToSpawn = 50000;
	static constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	TObjectPtr<UMassTestProcessorBase> CreateEntitiesProcessor = nullptr;

	FMassElementBitSet FloatsComposition;
	FMassElementBitSet LargeComposition;
	virtual bool PopulatePhasesConfig() override
	{
		return true;
	}

	virtual bool SetUp() override
	{
		if (!Super::SetUp())
		{
			return false;
		}

		TArray<FMassEntityHandle> Created;
		Created.Reserve(NumToSpawn);
		EntityManager->BatchCreateEntities(IntsArchetype, NumToSpawn, Created);

		FloatsComposition = FloatsArchetype.GetCompositionBitSetChecked();
		LargeComposition = LargeArchetype.GetCompositionBitSetChecked();

		CreateEntitiesProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateEntitiesProcessor->SetShouldAllowMultipleInstances(true);
		CreateEntitiesProcessor->SetProcessingPhase(PhaseToRun);
		CreateEntitiesProcessor->CallInitialize(GetTransientPackage(), EntityManager->AsShared());
		CreateEntitiesProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);

		CreateEntitiesProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				CreateEntitiesProcessor->EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
					{
						for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
						{
							Context.Defer().PushCommand<FMassDeferredCreateCommand>(
								[this](FMassEntityManager& DeferredEntityManager)
								{
									DeferredEntityManager.CreateEntity(FloatsArchetype);
								}
							);

							Context.Defer().PushCommand<FMassDeferredCreateCommand>(
								[this](FMassEntityManager& DeferredEntityManager)
								{
									DeferredEntityManager.CreateEntity(LargeArchetype);
								}
							);
						}
					});
			};
		PhaseManager->RegisterDynamicProcessor(*CreateEntitiesProcessor);

		return true;
	}

	virtual bool Update() override
	{
		Super::Update();
		return true;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();

#if WITH_MASSENTITY_DEBUG
		// Verify that entities ended up in IntsArchetype.
		const int32 IntsCount = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in IntsArchetype"), IntsCount, NumToSpawn);

		const int32 FloatsCount = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in FloatsArchetype"), FloatsCount, NumToSpawn);

		const int32 LargeCount = EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in LargeArchetype"), LargeCount, NumToSpawn);
#endif //WITH_MASSENTITY_DEBUG

		PhaseManager->UnregisterDynamicProcessor(*CreateEntitiesProcessor);
		CreateEntitiesProcessor->MarkAsGarbage();
		CreateEntitiesProcessor = nullptr;
	}
};
IMPLEMENT_AI_LATENT_TEST(FComplex_DeferredCreateEntitiesDuringQuery, "System.Mass.Complex.DeferredCreateEntitiesDuringQuery");


//-----------------------------------------------------------------------------
// Batched entity creation in scheduled processors
//-----------------------------------------------------------------------------
struct FComplex_BatchedCreateEntitiesParallel : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;

	static constexpr int32 NumToSpawn = 1000000;
	static constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	TObjectPtr<UMassTestProcessorBase> CreateIntsProcessor = nullptr;
	TObjectPtr<UMassTestProcessorBase> CreateFloatsProcessor = nullptr;
	TObjectPtr<UMassTestProcessorBase> CreateLargeProcessor = nullptr;

	FMassElementBitSet IntsComposition;
	FMassElementBitSet FloatsComposition;
	FMassElementBitSet LargeComposition;

	virtual bool PopulatePhasesConfig() override
	{
		return true;
	}

	virtual bool SetUp() override
	{
		if (!Super::SetUp())
		{
			return false;
		}

		IntsComposition = IntsArchetype.GetCompositionBitSetChecked();
		CreateIntsProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateIntsProcessor->SetShouldAllowMultipleInstances(true);
		CreateIntsProcessor->SetProcessingPhase(PhaseToRun);
		CreateIntsProcessor->GetEntityCreationRequirements().AddCreatedArchetype(IntsComposition);
		CreateIntsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Creating IntsArchetype");

				TArray<FMassEntityHandle> Created;
				Created.Reserve(NumToSpawn);
				EntityManager->BatchReserveEntities(NumToSpawn, Created);
				for (int32 i = 0; i < NumToSpawn; ++i)
				{
					Context.BuildEntity(IntsArchetype, Created[i]);
				}

				UE_LOGF(LogMassComplexTest, Log, "End Creating IntsArchetype");
			};
		PhaseManager->RegisterDynamicProcessor(*CreateIntsProcessor);

		FloatsComposition = FloatsArchetype.GetCompositionBitSetChecked();
		CreateFloatsProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateFloatsProcessor->SetShouldAllowMultipleInstances(true);
		CreateFloatsProcessor->SetProcessingPhase(PhaseToRun);
		CreateFloatsProcessor->GetEntityCreationRequirements().AddCreatedArchetype(FloatsComposition);
		CreateFloatsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Creating FloatsArchetype");

				TArray<FMassEntityHandle> Created;
				Created.Reserve(NumToSpawn);
				EntityManager->BatchReserveEntities(NumToSpawn, Created);
				for (int32 i = 0; i < NumToSpawn; ++i)
				{
					Context.BuildEntity(FloatsArchetype, Created[i]);
				}

				UE_LOGF(LogMassComplexTest, Log, "End Creating FloatsArchetype");
			};
		PhaseManager->RegisterDynamicProcessor(*CreateFloatsProcessor);

		LargeComposition = LargeArchetype.GetCompositionBitSetChecked();
		CreateLargeProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateLargeProcessor->SetShouldAllowMultipleInstances(true);
		CreateLargeProcessor->SetProcessingPhase(PhaseToRun);
		CreateLargeProcessor->GetEntityCreationRequirements().AddCreatedArchetype(LargeComposition);
		CreateLargeProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Creating LargeArchetype");

				TArray<FMassEntityHandle> Created;
				Created.Reserve(NumToSpawn);
				EntityManager->BatchReserveEntities(NumToSpawn, Created);
				for (int32 i = 0; i < NumToSpawn; ++i)
				{
					Context.BuildEntity(LargeArchetype, Created[i]);
				}

				UE_LOGF(LogMassComplexTest, Log, "End Creating LargeArchetype");
			};
		PhaseManager->RegisterDynamicProcessor(*CreateLargeProcessor);


		return true;
	}

	virtual bool Update() override
	{
		Super::Update();
		return true;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();

#if WITH_MASSENTITY_DEBUG
		// Verify that entities ended up in IntsArchetype.
		const int32 IntsCount = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in IntsArchetype"), IntsCount, NumToSpawn);

		const int32 FloatsCount = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in FloatsArchetype"), FloatsCount, NumToSpawn);

		const int32 LargeCount = EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in LargeArchetype"), LargeCount, NumToSpawn);
#endif //WITH_MASSENTITY_DEBUG

		PhaseManager->UnregisterDynamicProcessor(*CreateIntsProcessor);
		CreateIntsProcessor->MarkAsGarbage();
		CreateIntsProcessor = nullptr;
		PhaseManager->UnregisterDynamicProcessor(*CreateFloatsProcessor);
		CreateFloatsProcessor->MarkAsGarbage();
		CreateFloatsProcessor = nullptr;
		PhaseManager->UnregisterDynamicProcessor(*CreateLargeProcessor);
		CreateLargeProcessor->MarkAsGarbage();
		CreateLargeProcessor = nullptr;
	}
};
IMPLEMENT_AI_LATENT_TEST(FComplex_BatchedCreateEntitiesParallel, "System.Mass.Complex.BatchedCreateEntitiesParallel");

//-----------------------------------------------------------------------------
// Deferred entity creation to compare with immediate in processor API
//-----------------------------------------------------------------------------
struct FComplex_CreateEntitiesDeferred : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;

	static constexpr int32 NumToSpawn = 1000000;
	static constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	TObjectPtr<UMassTestProcessorBase> CreateIntsProcessor = nullptr;
	TObjectPtr<UMassTestProcessorBase> CreateFloatsProcessor = nullptr;
	TObjectPtr<UMassTestProcessorBase> CreateLargeProcessor = nullptr;

	// Keep the descriptors (used to build archetypes) and also store handles.
	FMassArchetypeCompositionDescriptor IntsComposition;
	FMassArchetypeCompositionDescriptor FloatsComposition;
	FMassArchetypeCompositionDescriptor LargeComposition;

	FMassArchetypeHandle IntsArchetypeHandle;
	FMassArchetypeHandle FloatsArchetypeHandle;
	FMassArchetypeHandle LargeArchetypeHandle;

	virtual bool PopulatePhasesConfig() override
	{
		return true;
	}

	virtual bool SetUp() override
	{
		if (!Super::SetUp())
		{
			return false;
		}

		CreateIntsProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateIntsProcessor->SetShouldAllowMultipleInstances(true);
		CreateIntsProcessor->SetProcessingPhase(PhaseToRun);
		CreateIntsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				Context.Defer().PushCommand<FMassDeferredCreateCommand>(
					[this](FMassEntityManager& DeferredEntityManager)
					{
						UE_LOGF(LogMassComplexTest, Log, "Begin Creating IntsArchetype");
						for (int32 i = 0; i < NumToSpawn; ++i)
						{
							DeferredEntityManager.CreateEntity(IntsArchetype);
						}
						UE_LOGF(LogMassComplexTest, Log, "End Creating IntsArchetype");
					}
				);
			};
		PhaseManager->RegisterDynamicProcessor(*CreateIntsProcessor);

		CreateFloatsProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateFloatsProcessor->SetShouldAllowMultipleInstances(true);
		CreateFloatsProcessor->SetProcessingPhase(PhaseToRun);
		CreateFloatsProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				Context.Defer().PushCommand<FMassDeferredCreateCommand>(
					[this](FMassEntityManager& DeferredEntityManager)
					{
						UE_LOGF(LogMassComplexTest, Log, "Begin Creating FloatsArchetype");
						for (int32 i = 0; i < NumToSpawn; ++i)
						{
							DeferredEntityManager.CreateEntity(FloatsArchetype);
						}
						UE_LOGF(LogMassComplexTest, Log, "End Creating FloatsArchetype");
					}
				);
			};
		PhaseManager->RegisterDynamicProcessor(*CreateFloatsProcessor);

		CreateLargeProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		CreateLargeProcessor->SetShouldAllowMultipleInstances(true);
		CreateLargeProcessor->SetProcessingPhase(PhaseToRun);
		CreateLargeProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				for (int32 i = 0; i < NumToSpawn; ++i)
				{
					Context.Defer().PushCommand<FMassDeferredCreateCommand>(
						[this](FMassEntityManager& DeferredEntityManager)
						{
							DeferredEntityManager.CreateEntity(LargeArchetype);
						}
					);
				}
			};
		PhaseManager->RegisterDynamicProcessor(*CreateLargeProcessor);

		return true;
	}

	virtual bool Update() override
	{
		Super::Update();
		return true;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();

#if WITH_MASSENTITY_DEBUG
		// After the phase runs, deferred commands are flushed; verify counts.
		const int32 IntsCount = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in IntsArchetype"), IntsCount, NumToSpawn);

		const int32 FloatsCount = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in FloatsArchetype"), FloatsCount, NumToSpawn);

		const int32 LargeCount = EntityManager->DebugGetArchetypeEntitiesCount(LargeArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected number of entities created in LargeArchetype"), LargeCount, NumToSpawn);
#endif //WITH_MASSENTITY_DEBUG

		PhaseManager->UnregisterDynamicProcessor(*CreateIntsProcessor);
		CreateIntsProcessor->MarkAsGarbage();
		CreateIntsProcessor = nullptr;
		PhaseManager->UnregisterDynamicProcessor(*CreateFloatsProcessor);
		CreateFloatsProcessor->MarkAsGarbage();
		CreateFloatsProcessor = nullptr;
		PhaseManager->UnregisterDynamicProcessor(*CreateLargeProcessor);
		CreateLargeProcessor->MarkAsGarbage();
		CreateLargeProcessor = nullptr;
	}
};
IMPLEMENT_AI_LATENT_TEST(FComplex_CreateEntitiesDeferred, "System.Mass.Complex.CreateEntitiesDeferred");

struct FComplex_DestroyEntitiesDuringQuery : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;

	static constexpr int32 NumToSpawn = 10;
	static constexpr EMassProcessingPhase PhaseToRun = EMassProcessingPhase::PrePhysics;

	TObjectPtr<UMassTestProcessorBase> DestroyEntitiesProcessor = nullptr;

	// Store the old CVar state to restore it later
	bool bOldFullyParallel = true;

	virtual bool PopulatePhasesConfig() override
	{
		return true;
	}

	virtual bool SetUp() override
	{
		// 1. Force Mass to run synchronously via CVar before Super::SetUp initializes the PhaseManager.
		if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("mass.FullyParallel")))
		{
			bOldFullyParallel = CVar->GetBool();
			CVar->Set(false, ECVF_SetByCode);
		}

		// Now initialize the environment (which reads the CVar)
		if (!Super::SetUp())
		{
			return false;
		}

		// 2. Spawn entities
		TArray<FMassEntityHandle> Created;
		Created.Reserve(NumToSpawn);
		EntityManager->BatchCreateEntities(IntsArchetype, NumToSpawn, Created);

		// 3. Setup the processor
		DestroyEntitiesProcessor = NewObject<UMassTestProcessorBase>(GetTransientPackage());
		DestroyEntitiesProcessor->SetShouldAllowMultipleInstances(true);
		DestroyEntitiesProcessor->SetProcessingPhase(PhaseToRun);
		DestroyEntitiesProcessor->CallInitialize(GetTransientPackage(), EntityManager->AsShared());

		// Note: We are NOT setting bRequiresMainThreadExecution = true here.
		// We rely on "mass.FullyParallel = false" to force the phase to run on the GameThread.

		DestroyEntitiesProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);

		DestroyEntitiesProcessor->ExecutionFunction = [this](FMassEntityManager& ExecuteEntityManager, FMassExecutionContext& Context)
			{
				UE_LOGF(LogMassComplexTest, Log, "Begin Destroying Entities (CVar Synchronous)");

				DestroyEntitiesProcessor->EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
					{
						for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
						{
							FMassEntityHandle Entity = Context.GetEntity(EntityIt);

							// Direct access to EntityManager (Main Buffer).
							// This is safe because mass.FullyParallel=false ensures this runs on the GameThread.
							Context.Defer().DestroyEntity(Entity);
						}
					});

				UE_LOGF(LogMassComplexTest, Log, "End Destroying Entities (CVar Synchronous)");
			};

		PhaseManager->RegisterDynamicProcessor(*DestroyEntitiesProcessor);

		return true;
	}

	virtual bool Update() override
	{
		Super::Update();
		return true;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();

#if WITH_MASSENTITY_DEBUG
		const int32 IntsCount = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
		AITEST_EQUAL_LATENT(TEXT("Expected IntsArchetype count to be 0 after synchronous destruction"), IntsCount, 0);
#endif //WITH_MASSENTITY_DEBUG

		PhaseManager->UnregisterDynamicProcessor(*DestroyEntitiesProcessor);
		DestroyEntitiesProcessor->MarkAsGarbage();
		DestroyEntitiesProcessor = nullptr;

		// Restore the CVar to avoid side effects on other tests
		if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("mass.FullyParallel")))
		{
			CVar->Set(bOldFullyParallel, ECVF_SetByCode);
		}
	}
};
IMPLEMENT_AI_LATENT_TEST(FComplex_DestroyEntitiesDuringQuery, "System.Mass.Complex.DestroyEntitiesDuringQuery");
} // namespace FMassComplexTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE