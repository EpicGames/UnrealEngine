// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassCompositeProcessorTest
{

struct FCompositeProcessorTest_Empty : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
		check(CompositeProcessor);
		FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
		// it should just run, no warnings
		UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeProcessorTest_Empty, "System.Mass.Processor.Composite.Empty");

/** 
 * This test ensures that no processors will do any actual work, if there are no entities matching their requirements. 
 */
struct FCompositeProcessorTest_NoWork : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
		check(CompositeProcessor);

		int TimesExecuted = 0;
		FMassExecuteFunction QueryExecFunction = [&TimesExecuted](FMassExecutionContext& Context)
		{
			++TimesExecuted;
		};
		
		{
			TArray<UMassProcessor*> Processors;
			for (int i = 0; i < 3; ++i)
			{
				UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
				Processor->ForEachEntityChunkExecutionFunction = QueryExecFunction;
				// need to set up some requirements to make EntityQuery valid
				Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
				Processors.Add(Processor);
			}

			CompositeProcessor->SetChildProcessors(Processors);
		}

		FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
		UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
		AITEST_EQUAL("None of the execution functions should have been executed", TimesExecuted, 0);

		// now test there being some entities but of different composition. We create Float entities, but processors
		// require Ints. 
		constexpr int32 NumberOfEntitieToCreate = 17;
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumberOfEntitieToCreate, EntitiesCreated);

		TimesExecuted = 0;
		{
			TArray<TObjectPtr<UMassProcessor>> Processors;
			for (int i = 0; i < 3; ++i)
			{
				UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
				Processor->ForEachEntityChunkExecutionFunction = QueryExecFunction;
				Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
				Processors.Add(Processor);
			}

			CompositeProcessor->SetChildProcessors(MoveTemp(Processors));
		}

		UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
		AITEST_EQUAL("None of the execution functions should have been executed", TimesExecuted, 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeProcessorTest_NoWork, "System.Mass.Processor.Composite.NoWork");

struct FCompositeProcessorTest_MultipleSubProcessors : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		// we create a single entity so that the processors' execution functions would get called.
		EntityManager->CreateEntity(IntsArchetype);

		UMassCompositeProcessor* CompositeProcessor = NewObject<UMassCompositeProcessor>();
		check(CompositeProcessor);

		int ExpectedResult = 0;
		int Result = 0;
		{
			TArray<UMassProcessor*> Processors;
			for (int i = 0; i < 3; ++i)
			{
				UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
				Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
				int Value = (int)FMath::Pow(10.f, float(i));
				Processor->ForEachEntityChunkExecutionFunction = [&Result, Value](FMassExecutionContext& Context)
				{
					Result += Value;
				};
				ExpectedResult += Value;

				Processors.Add(Processor);
			}

			CompositeProcessor->SetChildProcessors(Processors);
		}

		FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
		UE::Mass::Executor::Run(*CompositeProcessor, ProcessingContext);
		AITEST_EQUAL("All of the child processors should get run", Result, ExpectedResult);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeProcessorTest_MultipleSubProcessors, "System.Mass.Processor.Composite.MultipleSubProcessors");

} // FMassCompositeProcessorTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE

