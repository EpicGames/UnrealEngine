// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityTypes.h"
#include "MassExecutionContext.h"
#include "MassObserverNotificationTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::CreationContext
{
//-----------------------------------------------------------------------------
// creation context 
//-----------------------------------------------------------------------------
struct FCreationContextTest : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 IntEntitiesToSpawnCount = 6;
		constexpr int32 FloatEntitiesToSpawnCount = 7;

		TArray<FMassEntityHandle> Entities;
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContextInt = EntityManager->BatchCreateEntities(IntsArchetype, IntEntitiesToSpawnCount, Entities);
		TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContextFloat = EntityManager->BatchCreateEntities(FloatsArchetype, FloatEntitiesToSpawnCount, Entities);
		const int32 NumDifferentArchetypesUsed = 2;

		AITEST_EQUAL(TEXT("Two back to back entity creation operations should result in the same creation context"), CreationContextInt, CreationContextFloat);
		AITEST_TRUE(TEXT("CreationContext's entity collection should be still valid since we only created two consistent collections of entities")
			, CreationContextInt->DebugAreEntityCollectionsUpToDate());

		TArray<FMassArchetypeEntityCollection> EntityCollections = CreationContextInt->GetEntityCollections(*EntityManager.Get());
		AITEST_EQUAL(TEXT("We expect the number of resulting collections to match expectations"), EntityCollections.Num(), NumDifferentArchetypesUsed);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreationContextTest, "System.Mass.CreationContext.Append");

struct FManualCreate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 IntEntitiesToSpawnCount = 6;
		constexpr int32 FloatEntitiesToSpawnCount = 7;
		int NumDifferentArchetypesUsed = 0;

		TArray<FMassEntityHandle> Entities;
		TSharedRef<FMassEntityManager::FEntityCreationContext> ObtainedContext = EntityManager->GetOrMakeCreationContext();
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> ObtainedContextCopy = EntityManager->GetOrMakeCreationContext();
			AITEST_EQUAL(TEXT("Two back to back creation context fetching should result in the same instance"), ObtainedContext, ObtainedContextCopy);
		}

		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContextInt = EntityManager->BatchCreateEntities(IntsArchetype, IntEntitiesToSpawnCount, Entities);
			AITEST_EQUAL(TEXT("Creating entities should return the original context"), ObtainedContext, CreationContextInt);
			++NumDifferentArchetypesUsed;
		}
		
		AITEST_TRUE(TEXT("CreationContext's entity collection should be still valid at this moment since we only added one entity collection/array")
			, ObtainedContext->DebugAreEntityCollectionsUpToDate());

		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> TempContext = EntityManager->BatchCreateEntities(IntsArchetype, IntEntitiesToSpawnCount, Entities);
			AITEST_EQUAL(TEXT("Creating entities should return the original context"), ObtainedContext, TempContext);

			AITEST_TRUE(TEXT("CreationContext's entity collection should be still valid, because we're only piling up consistent entity collections")
				, TempContext->DebugAreEntityCollectionsUpToDate());
		}

		TArray<FMassArchetypeEntityCollection> EntityCollections = ObtainedContext->GetEntityCollections(*EntityManager.Get());
		AITEST_EQUAL(TEXT("We expect the number of resulting collections to match expectations"), EntityCollections.Num(), NumDifferentArchetypesUsed);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FManualCreate, "System.Mass.CreationContext.ManualCreate");

struct FManualBuild : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 FloatEntitiesToSpawnCount = 7;
		int NumDifferentArchetypesUsed = 0;

		TArray<FTestFragment_Float> Payload;
		for (int Index = 0; Index < FloatEntitiesToSpawnCount; ++Index)
		{ 
			Payload.Add(FTestFragment_Float(float(Index)));
		}

		TSharedRef<FMassEntityManager::FEntityCreationContext> ObtainedContext = EntityManager->GetOrMakeCreationContext();
		
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchReserveEntities(FloatEntitiesToSpawnCount, Entities);

		FStructArrayView PayloadView(Payload);
		TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
		FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(*EntityManager, Entities, FMassArchetypeEntityCollection::NoDuplicates
			, FMassGenericPayloadView(MakeArrayView(&PayloadView, 1)), EntityCollections);

		checkf(EntityCollections.Num() == 1, TEXT("We expect TargetEntities to only contain archetype-less entities, ones that need to be built"));

		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchBuildEntities(EntityCollections[0], FMassFragmentBitSet(FTestFragment_Float::StaticStruct()));
			AITEST_EQUAL(TEXT("Creating entities should return the original context"), ObtainedContext, CreationContext);
			++NumDifferentArchetypesUsed;
		}

		AITEST_TRUE(TEXT("CreationContext's entity collection should be still valid at this moment since we only added one entity collection/array")
			, ObtainedContext->DebugAreEntityCollectionsUpToDate());

		TArray<FMassArchetypeEntityCollection> ContextEntityCollections = ObtainedContext->GetEntityCollections(*EntityManager.Get());
		AITEST_EQUAL(TEXT("We expect the number of resulting collections to match expectations"), ContextEntityCollections.Num(), NumDifferentArchetypesUsed);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FManualBuild, "System.Mass.CreationContext.ManualBuild");

// Requires a fix for UE-333172
//struct FCreationContextScopeTest : FEntityTestBase
//{
//	virtual bool InstantTest() override
//	{
//		TArray<FMassEntityHandle> BothEntities;
//		constexpr int FirstEntityValue = 1;
//		constexpr int SecondEntityValue = 2;
//
//		{
//			TArray<FMassEntityHandle> Entities;
//			const TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, 1, Entities);
//			BothEntities.Append(Entities);
//
//			TArray<FMassArchetypeEntityCollection> EntityCollections = CreationContext->GetEntityCollections(*EntityManager.Get());
//			AITEST_EQUAL(TEXT("We expect the number of resulting collections to match expectations"), EntityCollections.Num(), 1);
//			AITEST_EQUAL(TEXT("We expect the number of entities in resulting collection to match expectations"), EntityCollections[0].DebugCountEntities(), 1);
//
//			EntityManager->BatchSetEntityFragmentValues(CreationContext->GetEntityCollections(*EntityManager), {FInstancedStruct::Make(FTestFragment_Int{FirstEntityValue})});
//		}
//
//		{
//			TArray<FMassEntityHandle> Entities;
//			const TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, 1, Entities);
//			BothEntities.Append(Entities);
//
//			TArray<FMassArchetypeEntityCollection> EntityCollections = CreationContext->GetEntityCollections(*EntityManager.Get());
//			AITEST_EQUAL(TEXT("We expect the number of resulting collections to match expectations"), EntityCollections.Num(), 1);
//			AITEST_EQUAL(TEXT("We expect the number of entities in resulting collection to match expectations"), EntityCollections[0].DebugCountEntities(), 1);
//
//			EntityManager->BatchSetEntityFragmentValues(CreationContext->GetEntityCollections(*EntityManager), {FInstancedStruct::Make(FTestFragment_Int{SecondEntityValue})});
//		}
//
//		AITEST_EQUAL(TEXT("We expect the total number of entities to match expectations"), BothEntities.Num(), 2);
//
//		FTestFragment_Int* FirstEntityFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BothEntities[0]);
//		AITEST_NOT_NULL(TEXT("The first entity is expected to have the int fragment"), FirstEntityFragment);
//		AITEST_EQUAL(TEXT("We expect the first entity's fragment to match expectations"), FirstEntityFragment->Value, FirstEntityValue);
//
//		FTestFragment_Int* SecondEntityFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BothEntities[1]);
//		AITEST_NOT_NULL(TEXT("The second entity is expected to have the int fragment"), SecondEntityFragment);
//		AITEST_EQUAL(TEXT("We expect the second entity's fragment to match expectations"), SecondEntityFragment->Value, SecondEntityValue);
//
//		return true;
//	}
//};
//IMPLEMENT_AI_INSTANT_TEST_WITH_FLAGS(FCreationContextScopeTest, "System.Mass.CreationContext.ScopedOperations", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::SupportsAutoRTFM)

} // UE::Mass::Test::CreationContext

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
