// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassProcessorDependencySolver.h"
#include "MassEntityTestTypes.h"
#include "MassTypeManager.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassDependencySolverTest
{

template<typename T>
static FName GetProcessorName()
{
	return T::StaticClass()->GetFName();
}

struct FDependencySolverBase : FAITestBase
{
	TArray<UMassTestProcessorBase*> Processors;
	TArray<FMassProcessorOrderInfo> Result;
	TSharedPtr<FMassEntityManager> EntityManager;
	
	virtual bool SetUp() override
	{
		EntityManager = MakeShareable(new FMassEntityManager());
		Processors.Reset();

		EntityManager->GetTypeManager().RegisterType<FTestSharedFragment_Int>();
		EntityManager->GetTypeManager().RegisterType<UMassTestWorldSubsystem>();
		EntityManager->GetTypeManager().RegisterType<UMassTestParallelSubsystem>();

		return true;
	}

	void Solve()
	{
		Result.Reset();
		FMassProcessorDependencySolver Solver(MakeArrayView((UMassProcessor**)Processors.GetData(), Processors.Num()));
		Solver.ResolveDependencies(Result, EntityManager);
	}
};	

struct FTrivialDependency : FDependencySolverBase
{
	UMassTestProcessorBase* Proc = nullptr;

	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		AITEST_EQUAL("The results should contain only a single processor", Result.Num(), 1);
		AITEST_EQUAL("The sole processor should be the one we've added", Result[0].Processor, Proc);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTrivialDependency, "System.Mass.Dependencies.Trivial");

struct FSimpleDependency : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_C>());
		}

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_A>());
		}

		Processors.Add(NewTestProcessor<UMassTestProcessor_C>(EntityManager));

		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		AITEST_TRUE("C is expected to be first", Result[0].Name == GetProcessorName<UMassTestProcessor_C>());
		AITEST_TRUE("A is expected to be second", Result[1].Name == GetProcessorName<UMassTestProcessor_A>());
		AITEST_TRUE("B is expected to be third", Result[2].Name == GetProcessorName<UMassTestProcessor_B>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSimpleDependency, "System.Mass.Dependencies.Simple");

struct FMissingDependency : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(TEXT("NonExistingDependency"));
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteBefore.Add(TEXT("NonExistingDependency2"));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_C>());
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_C>(EntityManager));
		}
		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		// event though there's no direct dependency between A and B due to declared dependencies on "NonExistingDependency"
		// B should come before A

		AITEST_TRUE("C is expected to be the first one", Result[0].Name == GetProcessorName<UMassTestProcessor_C>());
		AITEST_TRUE("Then B", Result[1].Name == GetProcessorName<UMassTestProcessor_B>());
		AITEST_TRUE("With A being last", Result[2].Name == GetProcessorName<UMassTestProcessor_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMissingDependency, "System.Mass.Dependencies.MissingDependencies");

struct FDeepGroup : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(TEXT("W.X.Y.Z"));
			Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("P.Q.R");
		}

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("W.X.Y.Z");
		}

		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		// dump all the group information from the Result collection for easier ordering testing
		for (int32 i = 0; i < Result.Num(); ++i)
		{
			if (Result[i].NodeType != FMassProcessorOrderInfo::EDependencyNodeType::Processor)
			{
				Result.RemoveAt(i--, EAllowShrinking::No);
			}
		}

		AITEST_TRUE("B is expected to be first", Result[0].Name == GetProcessorName<UMassTestProcessor_B>());
		AITEST_TRUE("A is expected to be second", Result[1].Name == GetProcessorName<UMassTestProcessor_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FDeepGroup, "System.Mass.Dependencies.DeepGroup");

struct FComplexScenario : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Z");
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(TEXT("X.Y"));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(UMassTestProcessor_E::StaticClass()->GetFName());
		}

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Y");
		}

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_C>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Y");
		}

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_D>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteBefore.Add(UMassTestProcessor_A::StaticClass()->GetFName());
			Proc->GetMutableExecutionOrder().ExecuteBefore.Add(TEXT("X.Y"));
		}

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_E>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteInGroup = TEXT("X.Z");
		}

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_F>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(UMassTestProcessor_A::StaticClass()->GetFName());
		}
		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		AITEST_TRUE("None of the processors should have been pruned", Result.Num() == Processors.Num());

		for (int32 i = 0; i < Result.Num(); ++i)
		{
			AITEST_EQUAL("We expect only processor nodes in the results", Result[i].NodeType, FMassProcessorOrderInfo::EDependencyNodeType::Processor);
		}

		AITEST_TRUE("D is the only fully dependency-less processor so should be first", Result[0].Name == GetProcessorName<UMassTestProcessor_D>());		
		AITEST_TRUE("B and C come next", (Result[1].Name == GetProcessorName<UMassTestProcessor_B>() || Result[2].Name == GetProcessorName<UMassTestProcessor_B>()) && (Result[1].Name == GetProcessorName<UMassTestProcessor_C>() || Result[2].Name == GetProcessorName<UMassTestProcessor_C>()));
		AITEST_TRUE("Following by E", Result[3].Name == GetProcessorName<UMassTestProcessor_E>());
		AITEST_TRUE("Then A", Result[4].Name == GetProcessorName<UMassTestProcessor_A>());
		AITEST_TRUE("F is last", Result[5].Name == GetProcessorName<UMassTestProcessor_F>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FComplexScenario, "System.Mass.Dependencies.Complex");

struct FThreadUnsafeWriteLinkedEntityFragment : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			// This creates the indirect fragment requirement but an indirect requirement alone is not valid
			// The query requirements are valid because it also adds a requirement for the link type (const shared fragment)
			Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_E>(EntityManager));
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_C>(EntityManager));
			Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_D>(EntityManager));
			Proc->EntityQuery.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_F>(EntityManager));
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		}

		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		int32 EmptyCount = 0;
		for (const FMassProcessorOrderInfo& OrderInfo : Result)
		{
			if (OrderInfo.Dependencies.IsEmpty())
			{
				EmptyCount++;
			}
		}

		AITEST_TRUE("Dependent processor chain should have exactly one independent processor", EmptyCount == 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FThreadUnsafeWriteLinkedEntityFragment, "System.Mass.Dependencies.ThreadUnsafeWriteLinkedEntityFragment");

struct FThreadUnsafeWriteIndirectFragment : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_E>(EntityManager));
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			// This creates ONLY the indirect fragment requirement but an indirect requirement alone is not valid
			Proc->EntityQuery.AddIndirectFragmentRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			// we must have another requirement of some sort, in practice we would want to access something with the entity handle for indirect access
			// making this read-only ensures that the solver is only resolving indirect contention on FTestFragment_Int
			Proc->EntityQuery.AddRequirement<FTestFragment_EntityHandle>(EMassFragmentAccess::ReadOnly);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->EntityQuery.AddIndirectFragmentRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Proc->EntityQuery.AddRequirement<FTestFragment_EntityHandle>(EMassFragmentAccess::ReadOnly);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_C>(EntityManager));
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
		}

		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		int32 EmptyCount = 0;
		for (const FMassProcessorOrderInfo& OrderInfo : Result)
		{
			if (OrderInfo.Dependencies.IsEmpty())
			{
				EmptyCount++;
			}
		}

		AITEST_TRUE("Dependent processor chain should have exactly one independent processor", EmptyCount == 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FThreadUnsafeWriteIndirectFragment, "System.Mass.Dependencies.ThreadUnsafeWriteIndirectFragment");

struct FThreadUnsafeWriteSubsystem : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->EntityQuery.AddSubsystemRequirement<UMassTestWorldSubsystem>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->EntityQuery.AddSubsystemRequirement<UMassTestWorldSubsystem>(EMassFragmentAccess::ReadWrite);
		}

		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		AITEST_TRUE("Dependency between processors is expected", Result[0].Dependencies.IsEmpty() != Result[1].Dependencies.IsEmpty());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FThreadUnsafeWriteSubsystem, "System.Mass.Dependencies.ThreadUnsafeWriteSubsystem");

struct FThreadSafeWriteSubsystem : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->EntityQuery.AddSubsystemRequirement<UMassTestParallelSubsystem>(EMassFragmentAccess::ReadWrite);
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->EntityQuery.AddSubsystemRequirement<UMassTestParallelSubsystem>(EMassFragmentAccess::ReadWrite);
		}

		return true;
	}

	virtual bool InstantTest() override
	{
		Solve();

		AITEST_TRUE("No dependency between processors is expected", Result[0].Dependencies.IsEmpty() && Result[1].Dependencies.IsEmpty());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FThreadSafeWriteSubsystem, "System.Mass.Dependencies.ThreadSafeWriteSubsystem");

struct FBadInput_Empty : FDependencySolverBase
{
	virtual bool InstantTest() override
	{
		Solve();
		AITEST_TRUE("Empty input should be handled gracefully", Result.Num() == 0);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBadInput_Empty, "System.Mass.Dependencies.BadInput.Empty");

struct FBadInput_Null : FDependencySolverBase
{
	virtual bool InstantTest() override
	{
		Processors.Reset();
		Processors.Add(nullptr);
		{
			MASS_SCOPED_ENSURE_TEST(TEXT("nullptr found in Processors"), 1);
			Solve();
		}
		AITEST_TRUE("Single nullptr input should be handled gracefully", Result.Num() == 0);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBadInput_Null, "System.Mass.Dependencies.BadInput.SingleNull");

struct FBadInput_MultipleNulls : FDependencySolverBase
{
	virtual bool InstantTest() override
	{
		Processors.Reset();
		Processors.Add(nullptr);
		Processors.Add(nullptr);
		Processors.Add(nullptr);
		{
			MASS_SCOPED_ENSURE_TEST(TEXT("nullptr found in Processors"), 3);
			Solve();
		}
		AITEST_TRUE("Multiple nullptr inputs should be handled gracefully", Result.Num() == 0);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBadInput_MultipleNulls, "System.Mass.Dependencies.BadInput.MultipleNulls");

struct FBadInput_MixedNulls : FDependencySolverBase
{
	virtual bool InstantTest() override
	{
		Processors.Reset();
		Processors.Add(nullptr);
		Processors.Add(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
		Processors.Add(nullptr);
		Processors.Add(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
		{
			MASS_SCOPED_ENSURE_TEST(TEXT("nullptr found in Processors"), 2);
			Solve();
		}
		AITEST_TRUE("Mixed nullptr and proper inputs should be handled gracefully", Result.Num() == 2);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBadInput_MixedNulls, "System.Mass.Dependencies.BadInput.NullsMixedIn");

struct FBadInput_Duplicates : FDependencySolverBase
{
	virtual bool InstantTest() override
	{
		Processors.Reset();
		Processors.Add(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
		Processors.Add(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
		Processors.Add(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
		{
			MASS_SCOPED_ENSURE_TEST(TEXT("already registered. Duplicates are not supported"), 2);
			Solve();
		}
		AITEST_TRUE("Duplicates in input should be handled gracefully", Result.Num() == 1);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBadInput_Duplicates, "System.Mass.Dependencies.BadInput.Duplicates");

struct FGroupNamesGeneration : FAITestBase
{
	virtual bool InstantTest() override
	{
		TArray<FString> SubGroupNames;
		FName EmptyName;
		FMassProcessorDependencySolver::CreateSubGroupNames(EmptyName, SubGroupNames);

		AITEST_TRUE("Empty group name is supported", SubGroupNames.Num() > 0);
		AITEST_TRUE("Empty group name handled like any other name", SubGroupNames[0] == EmptyName.ToString());
		
		FMassProcessorDependencySolver::CreateSubGroupNames(TEXT("X"), SubGroupNames);
		AITEST_TRUE("Trivial group name is supported", SubGroupNames.Num() > 0);
		AITEST_TRUE("Trivial group name shouldn\'t get decorated", SubGroupNames[0] == TEXT("X"));

		FMassProcessorDependencySolver::CreateSubGroupNames(TEXT("W.X.Y.Z"), SubGroupNames);
		AITEST_TRUE("Complex group name should be result in a number of group names equal to group name\'s depth", SubGroupNames.Num() == 4);
		AITEST_TRUE("Group name W.X.Y.Z should contain subgroup W", SubGroupNames.Find(TEXT("W")) != INDEX_NONE);
		AITEST_TRUE("Group name W.X.Y.Z should contain subgroup W.X", SubGroupNames.Find(TEXT("W.X")) != INDEX_NONE);
		AITEST_TRUE("Group name W.X.Y.Z should contain subgroup W.X.Y", SubGroupNames.Find(TEXT("W.X.Y")) != INDEX_NONE);
		AITEST_TRUE("Group name W.X.Y.Z should contain subgroup W.X.Y.Z", SubGroupNames.Find(TEXT("W.X.Y.Z")) != INDEX_NONE);
		AITEST_TRUE("Split up of group name W.X.Y.Z should result in a given order"
			, SubGroupNames[0] == TEXT("W") && SubGroupNames[1] == TEXT("W.X") && SubGroupNames[2] == TEXT("W.X.Y") && SubGroupNames[3] == TEXT("W.X.Y.Z"));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FGroupNamesGeneration, "System.Mass.Dependencies.SubgroupNames");

struct FCircularDependency : FDependencySolverBase
{
	virtual bool SetUp() override
	{
		FDependencySolverBase::SetUp();
		TSharedRef<FMassEntityManager> EntityManagerRef = EntityManager.ToSharedRef();

		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->EntityQuery.Initialize(EntityManagerRef);
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_D>());
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->EntityQuery.Initialize(EntityManagerRef);
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_A>());
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_C>(EntityManager));
			Proc->EntityQuery.Initialize(EntityManagerRef);
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_B>());
		}
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_D>(EntityManager));
			Proc->EntityQuery.Initialize(EntityManagerRef);
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_C>());
		}

		return true;
	}

	virtual bool InstantTest() override
	{
		{
			MASS_SCOPED_ENSURE_TEST(TEXT("Detected processing dependency cycle"), 1);
			MASS_SCOPED_ENSURE_TEST(TEXT("Encountered processing dependency cycle"), 1);
			// this one's added since we know we log these, and don't want to trip the detectors
			MASS_SCOPED_ENSURE_TEST(TEXT("group: None"), 4);
			Solve();
		}
		// every subsequent processor is expected to depend only on the previous one since all the processors use exactly the same resources
		AITEST_TRUE("The first processor has no dependencies", Result[0].Dependencies.IsEmpty());
		for (int i = 1; i < Result.Num(); ++i)
		{
			const auto& ResultNode = Result[i];
			AITEST_EQUAL("The subsequent processors has only one dependency", ResultNode.Dependencies.Num(), 1);
			AITEST_EQUAL("The subsequent processors depend only on the previous one", ResultNode.Dependencies[0], Result[i - 1].Name);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCircularDependency, "System.Mass.Dependencies.Circular");

} // FMassDependencySolverTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
