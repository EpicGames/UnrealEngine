// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UObjectGlobals.h"
#include "AITestsCommon.h"
#include "Engine/World.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassProcessingTypes.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassDynamicProcessorsTest
{

struct FDynamicProcessorsAddTrivial : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;
	int32 AllowedTicksCount = 2;
	int32 AddDynamicProcessorOnTickIndex = 1;
	int32 NumberOfTimesTicked = 0;
	UMassTestProcessorBase* DynamicProcessor = nullptr;
	TWeakObjectPtr<UMassTestProcessorBase> WeakDynamicProcessor;

	virtual bool PopulatePhasesConfig() override
	{
		// there are going to be no processors initially
		return true;
	}

	virtual bool Update() override
	{
		Super::Update();

		if (TickIndex == AddDynamicProcessorOnTickIndex)
		{
			// on second tick we're adding a dynamic processor
			DynamicProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
			DynamicProcessor->ExecutionFunction = [this](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
			{
				++NumberOfTimesTicked;
			};

			// if we Register the processor at this point we risk that the change will be picked up during the tick we just 
			// triggered via the Super::Update() call. We need to wait for those to ticks to be processed before we can continue
			if (CompletionEvent)
			{
				CompletionEvent->Wait();
			}
			PhaseManager->RegisterDynamicProcessor(*DynamicProcessor);
			WeakDynamicProcessor = DynamicProcessor;
		}

		// finishing after two ticks of the added dynamic processor
		return TickIndex >= AllowedTicksCount + AddDynamicProcessorOnTickIndex;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();
		AITEST_EQUAL_LATENT("Expecting the dynamic processor to be ticked the predicted number of times", NumberOfTimesTicked, AllowedTicksCount);
	}
};
IMPLEMENT_AI_LATENT_TEST(FDynamicProcessorsAddTrivial, "System.Mass.ProcessingPhases.DynamicProcessors.Add");


struct FDynamicProcessorsRemoveTrivial : FDynamicProcessorsAddTrivial
{
	using Super = FDynamicProcessorsAddTrivial;

	int32 ArbitraryNumberOfTicksAfterRemoval = 2;

	FDynamicProcessorsRemoveTrivial()
	{
		AllowedTicksCount = 1;
	}
	
	virtual bool Update() override
	{
		Super::Update();

		if (TickIndex == AddDynamicProcessorOnTickIndex + AllowedTicksCount)
		{
			// if we Register the processor at this point we risk that the change will be picked up during the tick we just 
			// triggered via the Super::Update() call. We need to wait for those to ticks to be processed before we can continue
			if (CompletionEvent)
			{
				CompletionEvent->Wait();
			}
			PhaseManager->UnregisterDynamicProcessor(*DynamicProcessor);
		}

		// finishing after two ticks of the added dynamic processor
		return TickIndex >= AllowedTicksCount + AddDynamicProcessorOnTickIndex + ArbitraryNumberOfTicksAfterRemoval;
	}
};
IMPLEMENT_AI_LATENT_TEST(FDynamicProcessorsRemoveTrivial, "System.Mass.ProcessingPhases.DynamicProcessors.Remove");


struct FDynamicProcessorsAddGCShield : FDynamicProcessorsAddTrivial
{
	using Super = FDynamicProcessorsAddTrivial;

	FDynamicProcessorsAddGCShield()
	{
		AllowedTicksCount = 5;
	}

	virtual bool Update() override
	{
		CA_ASSUME(GEngine);
		GEngine->Exec(World, TEXT("obj gc"));
		return Super::Update();
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();
		AITEST_NOT_NULL_LATENT("Expecting the dynamic processor to be valid regardless of multiple garbage collections", WeakDynamicProcessor.Get());
	}
};
IMPLEMENT_AI_LATENT_TEST(FDynamicProcessorsAddGCShield, "System.Mass.ProcessingPhases.DynamicProcessors.AddGCShield");


struct FDynamicProcessorsRemoveGCShield : FDynamicProcessorsRemoveTrivial
{
	using Super = FDynamicProcessorsRemoveTrivial;

	FDynamicProcessorsRemoveGCShield()
	{
		AllowedTicksCount = 5;
	}

	virtual bool Update() override
	{
		CA_ASSUME(GEngine);
		CollectGarbage(RF_NoFlags);
		return Super::Update();
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();
		AITEST_NOT_NULL_LATENT("Expecting the dynamic processor to have been set (test implementation verification)", DynamicProcessor);
		AITEST_NULL_LATENT("Expecting the dynamic processor to be removed by GC after we unregister it", WeakDynamicProcessor.Get());
	}
};
IMPLEMENT_AI_LATENT_TEST(FDynamicProcessorsRemoveGCShield, "System.Mass.ProcessingPhases.DynamicProcessors.RemoveGCShield");


struct FDynamicProcessorMultipleInstances : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;
	int32 AllowedTicksCount = 2;
	int32 AddDynamicProcessorOnTickIndex = 1;
	int32 NumberOfProcessorsToInstantiate = 3;
	// using atomic here since due to how the dynamic processors are being configured for this test will execute 
	// in parallel so the accumulation needs to be thread-safe
	std::atomic<int32> AccumulatedValue = 0;
	int32 ExpectedAccumulatedValue = 0;

	virtual bool PopulatePhasesConfig() override
	{
		// there are going to be no processors initially
		return true;
	}

	virtual bool Update() override
	{
		Super::Update();

		if (TickIndex == 1)
		{
			// if we Register the processor at this point we risk that the change will be picked up during the tick we just 
			// triggered via the Super::Update() call. We need to wait for those to ticks to be processed before we can continue
			if (CompletionEvent)
			{
				CompletionEvent->Wait();
			}

			for (int i = 0; i < NumberOfProcessorsToInstantiate; ++i)
			{
				UMassTestProcessorBase* DynamicProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
				DynamicProcessor->SetShouldAllowMultipleInstances(true);
				DynamicProcessor->ExecutionFunction = [this, i](FMassEntityManager& InEntityManager, FMassExecutionContext& Context)
				{
					AccumulatedValue += (i + 1);
				};
				ExpectedAccumulatedValue += (i + 1);

				PhaseManager->RegisterDynamicProcessor(*DynamicProcessor);
			}
			ExpectedAccumulatedValue *= AllowedTicksCount;
		}

		return TickIndex >= AllowedTicksCount + AddDynamicProcessorOnTickIndex;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();
		AITEST_EQUAL_LATENT("Expecting the accumulated value to match the prediction", AccumulatedValue.load(), ExpectedAccumulatedValue);
	}
};
IMPLEMENT_AI_LATENT_TEST(FDynamicProcessorMultipleInstances, "System.Mass.ProcessingPhases.DynamicProcessors.MultipleInstances");

} // FMassDynamicProcessorsTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE

