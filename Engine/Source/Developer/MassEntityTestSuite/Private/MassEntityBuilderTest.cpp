// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityTestTypes.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"


#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

/**
 * Tests to be added:
 * - test observers triggering as expected, i.e. respecting the construction context
 * - entity grouping
 */

namespace UE::Mass::Test::EntityBuilder
{
	struct FSimpleBuild : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
#if  WITH_MASSENTITY_DEBUG
			int32 EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
			int32 SomeCounter = 1;
			int32 EntitiesCreatedThisStep = 0;

			{
				FEntityBuilder EntityBuilder(*EntityManager.Get());
				EntityBuilder.SetForceDeferredCommit(false)
					.Add_GetRef<FTestFragment_Int>(SomeCounter++);
				EntityBuilder.Commit();
				EntitiesCreatedThisStep = 1;
			}
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("Number of entities created with basic use", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated, EntitiesCreatedThisStep);
			EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
			{
				FEntityBuilder EntityBuilder(*EntityManager.Get());
				EntityBuilder.SetForceDeferredCommit(false);

				ON_SCOPE_EXIT
				{
					EntityBuilder.Commit();
				};

				EntityBuilder.Add_GetRef<FTestFragment_Int>(SomeCounter++);
			}
			EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("Number of entities created with ON_SCOPE_EXIT", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated, EntitiesCreatedThisStep);
			EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
			{
				FScopedEntityBuilder EntityBuilder(*EntityManager.Get());
				EntityBuilder.SetForceDeferredCommit(false)
					.Add_GetRef<FTestFragment_Int>(SomeCounter++);
			}
			EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("Number of entities created with scoped builder", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated, EntitiesCreatedThisStep);
			EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
			{
				FScopedEntityBuilder EntityBuilder(*EntityManager.Get());
				EntityBuilder.SetForceDeferredCommit(false)
					.Add_GetRef<FTestFragment_Int>(SomeCounter++);

				FEntityBuilder EntityBuilder2 = EntityBuilder;
				EntityBuilder2.Commit();
			}
			EntitiesCreatedThisStep = 2;
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("Number of entities created with with a scoped builder and its regular copy", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated, EntitiesCreatedThisStep);
			EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
			{
				FEntityBuilder EntityBuilder(*EntityManager.Get());
				EntityBuilder.SetForceDeferredCommit(false)
					.Add_GetRef<FTestFragment_Int>(SomeCounter++);
				EntityBuilder.Commit();

				FEntityBuilder EntityBuilder2 = EntityBuilder;
			}
			EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("Number of entities created with Commit and an abandoned copy of a builder", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated, EntitiesCreatedThisStep);
			EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
			{
				FEntityBuilder EntityBuilder(*EntityManager.Get());
				EntityBuilder.SetForceDeferredCommit(false)
					.Add_GetRef<FTestFragment_Int>(SomeCounter++);

				FEntityBuilder EntityBuilder2 = EntityBuilder;
				EntityBuilder2.Commit();

				EntityBuilder.Reset();
			}
			EntitiesCreatedThisStep = 1;
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("Number of entities created with committed copy and abandoned original builder", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) - EntitiesCreated, EntitiesCreatedThisStep);
			EntitiesCreated = EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype);
#endif // WITH_MASSENTITY_DEBUG
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSimpleBuild, "System.Mass.EntityBuilder.SimpleBuild");

	struct FAbort: FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			{
				FEntityBuilder Builder(EntityManager.ToSharedRef());
				Builder.Add<FTestFragment_Int>();
				FMassEntityHandle ReservedEntityHandle = Builder.GetEntityHandle();
				{
					const bool bValid = EntityManager->IsEntityValid(ReservedEntityHandle);
					AITEST_TRUE("Before committing the entity handle is reserved", bValid);
					const bool bIsBuilt = EntityManager->IsEntityActive(ReservedEntityHandle);
					AITEST_FALSE("Before committing the entity is already created", bIsBuilt);
				}
				Builder.Reset();
				{
					const bool bValid = EntityManager->IsEntityValid(ReservedEntityHandle);
					AITEST_FALSE("After resetting the entity handle is still valid", bValid);
				}
			}
			FMassEntityHandle AbandonedEntityHandle;
			{
				FEntityBuilder Builder(EntityManager.ToSharedRef());
				Builder.Add<FTestFragment_Int>();
				AbandonedEntityHandle = Builder.GetEntityHandle();
			}
			{
				const bool bValid = EntityManager->IsEntityValid(AbandonedEntityHandle);
				AITEST_FALSE("After builder's destruction without committing the entity handle is still valid", bValid);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FAbort, "System.Mass.EntityBuilder.Abort");

	struct FOneliner: FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			int32 TotalCountCreated = 0;
			{
				FMassEntityHandle CreatedEntity = EntityManager->MakeEntityBuilder()
					.SetForceDeferredCommit(false)
					.Add<FTestFragment_Int>().Commit();
				++TotalCountCreated;

				const bool bIsBuilt = EntityManager->IsEntityActive(CreatedEntity);
				AITEST_TRUE("The entity has been created", bIsBuilt);
#if WITH_MASSENTITY_DEBUG
				AITEST_TRUE("Only a single entity has been created", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == TotalCountCreated);
#endif // WITH_MASSENTITY_DEBUG
			}
			{
				FEntityBuilder EntityBuilder = EntityManager->MakeEntityBuilder()
					.SetForceDeferredCommit(false)
					.Add<FTestFragment_Int>();
				EntityBuilder.Commit();
				++TotalCountCreated;
			}
#if WITH_MASSENTITY_DEBUG
			AITEST_TRUE("The number of entities created matches expectations", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) == TotalCountCreated);
#endif // WITH_MASSENTITY_DEBUG
			{
				// we're not committing so this builder won't create an entity.
				EntityManager->MakeEntityBuilder().Add<FTestFragment_Int>();

				// similarly here, even reserving the entity won't result in building that entity without manual `Commit` call.
				FMassEntityHandle ReservedEntity = EntityManager->MakeEntityBuilder().Add<FTestFragment_Int>().GetEntityHandle();
			}
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("The number of entities created after not committing builders", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), TotalCountCreated);
#endif // WITH_MASSENTITY_DEBUG

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FOneliner, "System.Mass.EntityBuilder.OneLiner");

	struct FCopyBuilder : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const int32 ValueA = 1;
			FEntityBuilder BuilderA(*EntityManager.Get());
			BuilderA.SetForceDeferredCommit(false);
			BuilderA.Add<FTestFragment_Int>(ValueA);

			const int32 ValueB = ValueA + 1;
			FEntityBuilder BuilderB = EntityManager->MakeEntityBuilder();
			BuilderB.SetForceDeferredCommit(false);
			BuilderB.Add<FTestFragment_Int>(ValueB);

			// a different way of setting the value
			FEntityBuilder BuilderC = BuilderA;
			BuilderC.GetOrCreate<FTestFragment_Int>().Value = ValueB;

			BuilderA.Commit();
			BuilderB.Commit();
			BuilderC.Commit();

			FTestFragment_Int* FragmentA = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BuilderA.GetEntityHandle());
			AITEST_NOT_NULL("The original entity has the expected fragment", FragmentA);
			AITEST_EQUAL("The value of the original entity's fragment matches expectations", FragmentA->Value, ValueA);

			FTestFragment_Int* FragmentB = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BuilderB.GetEntityHandle());
			AITEST_NOT_NULL("The copied entity has the expected fragment", FragmentB);
			AITEST_EQUAL("The value of the copied entity's fragment matches expectations", FragmentB->Value, ValueB);

			FTestFragment_Int* FragmentC = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(BuilderC.GetEntityHandle());
			AITEST_NOT_NULL("The other copied entity has the expected fragment", FragmentC);
			AITEST_EQUAL("The value of the other copied entity's fragment matches expectations", FragmentC->Value, FragmentB->Value);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FCopyBuilder, "System.Mass.EntityBuilder.Copy");

	struct FOverride : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			FEntityBuilder BuilderA(*EntityManager.Get());
			BuilderA.Add<FTestFragment_Int>();

			FEntityBuilder BuilderB(*EntityManager.Get());
			BuilderB.Add<FTestFragment_Float>();

			FEntityBuilder BuilderC = BuilderB;
			
			FMassEntityHandle EntityA = BuilderA.GetEntityHandle();
			FMassEntityHandle EntityB = BuilderB.GetEntityHandle();
			FMassEntityHandle EntityC = BuilderC.GetEntityHandle();

			AITEST_NOT_EQUAL("Entities reserved by different builders, A|B", EntityA, EntityB);
			AITEST_NOT_EQUAL("Entities reserved by different builders, A|C", EntityA, EntityC);
			AITEST_NOT_EQUAL("Entities reserved by different builders, B|C", EntityB, EntityC);

			// the following operation is expected to stomp the settings of the target builder, but not the entity
			BuilderB = BuilderA;
			FMassEntityHandle EntityB2 = BuilderB.GetEntityHandle();
			AITEST_TRUE("The uncommitted (i.e. reserved) entity handle does not change with builder's config override", EntityB == EntityB2);

			// overriding a committed builder results in creation of a new handle.
			BuilderC.Commit();
			BuilderC = BuilderA;
			FMassEntityHandle EntityC2 = BuilderC.GetEntityHandle();
			AITEST_TRUE("The committed entity handle differs from the new one", EntityC != EntityC2);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FOverride, "System.Mass.EntityBuilder.Override");

	struct FPassOver : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			{
				FEntityBuilder BuilderA(*EntityManager.Get());
				BuilderA.Add<FTestFragment_Int>();
				// forces handle reservation
				FMassEntityHandle EntityA = BuilderA.GetEntityHandle();

				FMassEntityHandle EntityB;
				{
					FEntityBuilder BuilderB(*EntityManager.Get());
					BuilderB.Add<FTestFragment_Float>();
					EntityB = BuilderB.GetEntityHandle();
					BuilderA = MoveTemp(BuilderB);
				}

				// at this point EntityB should be valid while the original EntityA not
				AITEST_TRUE("The original entity is invalid", EntityManager->IsEntityValid(EntityA) == false);
				AITEST_TRUE("The passed-over entity entity is valid", EntityManager->IsEntityValid(EntityB));
			}

			{
				FEntityBuilder BuilderA(*EntityManager.Get());
				BuilderA.SetForceDeferredCommit(false);
				BuilderA.Add<FTestFragment_Int>();
				// forces handle reservation
				FMassEntityHandle EntityA = BuilderA.Commit();

				FMassEntityHandle EntityB;
				{
					FEntityBuilder BuilderB(*EntityManager.Get());
					BuilderB.Add<FTestFragment_Float>();
					EntityB = BuilderB.GetEntityHandle();
					BuilderA = MoveTemp(BuilderB);
				}

				// at this point both EntityA and EntityB should be valid
				AITEST_TRUE("The original entity is valid, since it was committed", EntityManager->IsEntityValid(EntityA));
				AITEST_TRUE("The original entity is active", EntityManager->IsEntityActive(EntityA));
				AITEST_TRUE("The passed-over entity entity is valid", EntityManager->IsEntityValid(EntityB));
			}

			{
				FEntityBuilder BuilderA(*EntityManager.Get());
				BuilderA.Add<FTestFragment_Int>();
				// forces handle reservation
				FMassEntityHandle EntityA = BuilderA.GetEntityHandle();

				FMassEntityHandle EntityB;
				{
					FEntityBuilder BuilderB(*EntityManager.Get());
					BuilderB.SetForceDeferredCommit(false);
					BuilderB.Add<FTestFragment_Float>();
					EntityB = BuilderB.Commit();
					BuilderA = MoveTemp(BuilderB);
				}

				// at this point both EntityA and EntityB should be valid, but only EntityB should be active (commited).
				AITEST_TRUE("The original entity is valid", EntityManager->IsEntityValid(EntityA));
				AITEST_TRUE("The original entity is NOT active", EntityManager->IsEntityActive(EntityA) == false);
				AITEST_TRUE("The secondary entity entity is valid", EntityManager->IsEntityValid(EntityB));
				AITEST_TRUE("The secondary entity is active", EntityManager->IsEntityActive(EntityB));
			}

			{
				FEntityBuilder BuilderA(*EntityManager.Get());
				BuilderA.SetForceDeferredCommit(false)
					.Add<FTestFragment_Int>();
				// forces handle reservation
				FMassEntityHandle EntityA = BuilderA.Commit();

				FMassEntityHandle EntityB;
				{
					FEntityBuilder BuilderB(*EntityManager.Get());
					BuilderB.SetForceDeferredCommit(false)
						.Add<FTestFragment_Float>();
					EntityB = BuilderB.Commit();
					BuilderA = MoveTemp(BuilderB);
				}

				// at this point both EntityA and EntityB should be valid and active (commited). 
				AITEST_TRUE("The original entity is valid", EntityManager->IsEntityValid(EntityA));
				AITEST_TRUE("The original entity is active", EntityManager->IsEntityActive(EntityA));
				AITEST_TRUE("The secondary entity entity is valid", EntityManager->IsEntityValid(EntityB));
				AITEST_TRUE("The secondary entity is active", EntityManager->IsEntityActive(EntityB));

				{
					MASS_SCOPED_ENSURE_TEST("Trying to commit an already committed", 1);
					BuilderA.Commit();
				}
				FMassEntityHandle EntityA2 = BuilderA.GetEntityHandle();
				AITEST_TRUE("The entity handle is the same as the builder's that has been moved", EntityA2 == EntityB);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FPassOver, "System.Mass.EntityBuilder.PassOver");

	struct FBuilderReuse : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			FEntityBuilder Builder = EntityManager->MakeEntityBuilder();
			Builder.SetForceDeferredCommit(false)
				.Add<FTestFragment_Int>();

			auto TestBuilder = [this, &Builder](const FMassArchetypeHandle& ExpectedArchetype)-> bool
			{
				TArray<const FMassEntityHandle> Entities = {
					Builder.CommitAndReprepare()
					, Builder.CommitAndReprepare()
				};

				const FMassArchetypeHandle LocalArchetype = Builder.GetArchetypeHandle();
				AITEST_NOT_EQUAL("Two entities created sequentially", Entities[0], Entities[1]);
				AITEST_EQUAL("Entities' archetype", EntityManager->GetArchetypeForEntity(Entities[0]), EntityManager->GetArchetypeForEntity(Entities[1]));
				AITEST_EQUAL("Builders archetype and entities' archetype", Builder.GetArchetypeHandle(), EntityManager->GetArchetypeForEntity(Entities[0]))
				AITEST_TRUE("Archetype matches expectations", EntityManager->GetArchetypeForEntity(Entities[0]) == ExpectedArchetype);
				return true;
			};

			if (TestBuilder(IntsArchetype))
			{
				Builder.Add<FTestFragment_Float>();
				return TestBuilder(FloatsIntsArchetype);
			}

			return false;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FBuilderReuse, "System.Mass.EntityBuilder.Reuse");

	struct FDuringProcessing : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumIterations = 5;
			// creating a single entity to enforce the execution function of the processor we're going to use to execute
			// exactly once
			EntityManager->CreateEntity(IntsArchetype);

			TArray<FMassEntityHandle> EntityHandles;

			constexpr int32 InitialValueToSet = 100;
			int32 ValueToSet = InitialValueToSet;
			UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
			Processor->ForEachEntityChunkExecutionFunction = [&ValueToSet, &EntityHandles](FMassExecutionContext& Context)
			{
				FEntityBuilder AsyncBuilder = Context.GetEntityManagerChecked().MakeEntityBuilder();
				AsyncBuilder.Add<FTestFragment_Int>(ValueToSet++);
				EntityHandles.Add(AsyncBuilder.Commit());
			};
			Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);

			FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f, /*bFlushCommandBuffer=*/false);

			for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
			{
				Executor::RunProcessorsView(MakeArrayView(reinterpret_cast<UMassProcessor**>(&Processor), 1), ProcessingContext);
				AITEST_EQUAL(FString::Printf(TEXT("Number of entities after iteration %d"), Iteration), EntityHandles.Num(), Iteration + 1);
			}

			for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
			{
				AITEST_FALSE(FString::Printf(TEXT("(NOT) Entity %d is `created`"), Iteration), EntityManager->IsEntityBuilt(EntityHandles[Iteration]));
			}

			EntityManager->FlushCommands();

			for (int32 Iteration = 0; Iteration < NumIterations; ++Iteration)
			{
				AITEST_TRUE(FString::Printf(TEXT("Entity %d is `created`"), Iteration), EntityManager->IsEntityBuilt(EntityHandles[Iteration]));
				AITEST_TRUE(FString::Printf(TEXT("Entity %d has the right archetype"), Iteration), EntityManager->GetArchetypeForEntity(EntityHandles[Iteration]) == IntsArchetype);
				if (Iteration + 1 < NumIterations)
				{
					AITEST_FALSE(FString::Printf(TEXT("(NOT) Entity handles are the same %d"), Iteration), EntityHandles[Iteration] == EntityHandles[Iteration + 1]);
				}
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FDuringProcessing, "System.Mass.EntityBuilder.DuringProcessing");

	struct FSyncBuildingAsyncSubmission : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			// creating a single entity to enforce the execution function of the processor we're going to use to execute
			// exactly once
			EntityManager->CreateEntity(IntsArchetype);

			FEntityBuilder SyncBuilder = EntityManager->MakeEntityBuilder();
			SyncBuilder.SetForceDeferredCommit(false)
				.Add<FTestFragment_Int>();

			const FMassEntityHandle ReservedHandle = SyncBuilder.GetEntityHandle();

			int32 ProcessedEntitiesCount = 0;
			UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
			Processor->ForEachEntityChunkExecutionFunction = [&SyncBuilder, &ProcessedEntitiesCount](FMassExecutionContext& Context)
			{
				SyncBuilder.Commit();
				ProcessedEntitiesCount += Context.GetNumEntities();
			};
			Processor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);

			FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f, /*bFlushCommandBuffer=*/false);
			Executor::RunProcessorsView(MakeArrayView(reinterpret_cast<UMassProcessor**>(&Processor), 1), ProcessingContext);

			AITEST_EQUAL("Number of fully-formed entities expected", ProcessedEntitiesCount, 1);
			AITEST_EQUAL("The entity handle before and after async commit", ReservedHandle, SyncBuilder.GetEntityHandle());
			AITEST_TRUE("The Builder is in `Committed` state", SyncBuilder.IsCommitted());
			// since the commands are not flushed yet, due to ProcessingContext's values, we expect the entity to not be created yet
			AITEST_FALSE("(NOT) the entity has been created", EntityManager->IsEntityBuilt(ReservedHandle));

			{
				MASS_SCOPED_ENSURE_TEST("Trying to commit an already committed", 1);
				AITEST_INFO("Second execution of the processor shouldn't change a thing.");
				Executor::RunProcessorsView(MakeArrayView(reinterpret_cast<UMassProcessor**>(&Processor), 1), ProcessingContext);
			}

			AITEST_EQUAL("Run 2: The entity handle before and after async commit", ReservedHandle, SyncBuilder.GetEntityHandle());
			AITEST_TRUE("Run 2: The Builder is in `Committed` state", SyncBuilder.IsCommitted());
			// since the commands are not flushed yet, due to ProcessingContext's values, we expect the entity to not be created yet
			AITEST_FALSE("(NOT) Run 2: the entity has been created", EntityManager->IsEntityBuilt(ReservedHandle));

			EntityManager->FlushCommands();
#if WITH_MASSENTITY_DEBUG
			AITEST_EQUAL("Number of entities in the target archetype, after flushing", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 2);
#endif // WITH_MASSENTITY_DEBUG
			AITEST_TRUE("the entity has been created", EntityManager->IsEntityBuilt(ReservedHandle));

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSyncBuildingAsyncSubmission, "System.Mass.EntityBuilder.SyncBuildingAsyncSubmission");

	struct FAllElementsUsed : FEntityTestBase
	{
		FMassEntityHandle OriginalEntity;
		const int32 TestIntValue = 17;
		const float TestFloatValue = 3.1415f;
		const float TestSharedFloatValue = 2.71828f;
		const int32 TestSharedIntValue = 1009;

		virtual bool SetUp() override
		{
			if (FEntityTestBase::SetUp())
			{
				// quick builder just to create an entity with known properties
				FEntityBuilder Builder(EntityManager.ToSharedRef());
				Builder.SetForceDeferredCommit(false);
				Builder.Add<FTestFragment_Int>(TestIntValue);
				Builder.Add<FTestFragment_Float>(TestFloatValue);
				Builder.Add<FTestTag_B>();
				Builder.Add<FTestSharedFragment_Float>(TestSharedFloatValue);
				Builder.Add<FTestConstSharedFragment_Int>(TestSharedIntValue);

				OriginalEntity = Builder.Commit();

				return true;
			}
			return false;
		}

		virtual bool InstantTest() override
		{
			FMassArchetypeCompositionDescriptor PredictedComposition;
			PredictedComposition.Add<FTestFragment_Int>();
			PredictedComposition.Add<FTestFragment_Float>();
			PredictedComposition.Add<FTestTag_B>();
			PredictedComposition.Add<FTestSharedFragment_Float>();
			PredictedComposition.Add<FTestConstSharedFragment_Int>();

			// testing composition
			const FMassArchetypeHandle ArchetypeHandle = EntityManager->GetArchetypeForEntity(OriginalEntity);
			const FMassArchetypeCompositionDescriptor& ArchetypeComposition = EntityManager->GetArchetypeComposition(ArchetypeHandle);
			AITEST_TRUE("Resulting archetype composition matches prediction", ArchetypeComposition.IsEquivalent(PredictedComposition));

			return TestEntity(OriginalEntity);
		}

		bool TestEntity(const FMassEntityHandle TestedEntity) const
		{
			{
				FTestFragment_Int* IntFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Int>(TestedEntity);
				AITEST_NOT_NULL("Created entity has the int fragment", IntFragmentInstance);
				AITEST_EQUAL("Resulting Int fragment value", IntFragmentInstance->Value, TestIntValue);
			}
			{
				FTestFragment_Float* FloatFragmentInstance = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(TestedEntity);
				AITEST_NOT_NULL("Created entity has the int fragment", FloatFragmentInstance);
				AITEST_EQUAL("Resulting Int fragment value", FloatFragmentInstance->Value, TestFloatValue);
			}
			{
				FTestSharedFragment_Float* SharedFragmentInstance = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(TestedEntity);
				AITEST_NOT_NULL("Created entity has the int fragment", SharedFragmentInstance);
				AITEST_EQUAL("Resulting Int fragment value", SharedFragmentInstance->Value, TestSharedFloatValue);
			}
			{
				FTestConstSharedFragment_Int* ConstSharedFragmentInstance = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(TestedEntity);
				AITEST_NOT_NULL("Created entity has the int fragment", ConstSharedFragmentInstance);
				AITEST_EQUAL("Resulting Int fragment value", ConstSharedFragmentInstance->Value, TestSharedIntValue);
			}
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FAllElementsUsed, "System.Mass.EntityBuilder.AllElements");

	struct FCopyEntity : FAllElementsUsed
	{
		virtual bool InstantTest() override
		{
			FEntityBuilder Builder(EntityManager.ToSharedRef());
			Builder.SetForceDeferredCommit(false);
			Builder.CopyDataFromEntity(OriginalEntity);

			const FMassEntityHandle NewEntityHandle = Builder.Commit();

			AITEST_TRUE("Source and target entities are in the same archetype"
				, EntityManager->GetArchetypeForEntity(OriginalEntity) == EntityManager->GetArchetypeForEntity(NewEntityHandle));

			return TestEntity(NewEntityHandle);
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FCopyEntity, "System.Mass.EntityBuilder.CopyEntity");

	struct FAppendFromEntity : FAllElementsUsed
	{
		virtual bool InstantTest() override
		{
			FEntityBuilder Builder(EntityManager.ToSharedRef());
			// adding something the appending won't add, just to remove it later and test the result
			Builder.Add<FTestTag_A>();
			Builder.AppendDataFromEntity(OriginalEntity);

			const FMassEntityHandle NewEntityHandle = Builder.SetForceDeferredCommit(false).Commit();

			EntityManager->RemoveTagFromEntity(NewEntityHandle, FTestTag_A::StaticStruct());

			AITEST_TRUE("Source and target entities are in the same archetype"
				, EntityManager->GetArchetypeForEntity(OriginalEntity) == EntityManager->GetArchetypeForEntity(NewEntityHandle));

			return TestEntity(NewEntityHandle);
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FAppendFromEntity, "System.Mass.EntityBuilder.Append");

	struct FUsingInstancedStructs : FAllElementsUsed
	{
		virtual bool SetUp() override
		{
			// deliberately skipping FAllElementsUsed, we're doing a different setup here
			if (FEntityTestBase::SetUp())
			{
				// quick builder just to create an entity with known properties
				FEntityBuilder Builder(EntityManager.ToSharedRef());
				Builder.SetForceDeferredCommit(false);

				FInstancedStruct ElementInstance;

				// adding "wrong value" first to verify that the last "add" will hold
				ElementInstance.InitializeAs<FTestFragment_Int>();
				Builder.Add(ElementInstance);
				ElementInstance.InitializeAs<FTestFragment_Int>(TestIntValue);
				Builder.Add(ElementInstance);
				
				ElementInstance.InitializeAs<FTestFragment_Float>();
				Builder.Add(MoveTemp(ElementInstance));
				ElementInstance.InitializeAs<FTestFragment_Float>(TestFloatValue);
				Builder.Add(MoveTemp(ElementInstance));

				ElementInstance.InitializeAs<FTestSharedFragment_Float>();
				Builder.Add(MoveTemp(ElementInstance));
				ElementInstance.InitializeAs<FTestSharedFragment_Float>(TestSharedFloatValue);
				Builder.Add(MoveTemp(ElementInstance));

				ElementInstance.InitializeAs<FTestConstSharedFragment_Int>();
				Builder.Add(ElementInstance);
				ElementInstance.InitializeAs<FTestConstSharedFragment_Int>(TestSharedIntValue);
				Builder.Add(ElementInstance);

				// tags cannot be added as instanced structs
				Builder.Add<FTestTag_B>();

				OriginalEntity = Builder.Commit();

				return true;
			}
			return false;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FUsingInstancedStructs, "System.Mass.EntityBuilder.WithInstancedStructs");
} // UE::Mass::Test::EntityBuilder

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
