// Copyright Epic Games, Inc. All Rights Reserved.

//#include "MassArchetypeData.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "Algo/Compare.h"
#include "MassEntityCollection.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//
namespace UE::Mass::Test::EntityCollection
{

struct FEntityCollectionTestBase : FEntityTestBase
{
	TArray<FMassEntityHandle> FloatEntities;
	TArray<FMassEntityHandle> IntEntities;
	TArray<FMassEntityHandle> FloatIntEntities;
	int32 EntitiesToCreatePerArchetype = 100;
	FRandomStream RandomStream = 0;
	static constexpr int32 NumArchetypesUsed = 3;
	static constexpr int32 NumTestedEntities = 50;

	TConstArrayView<FMassEntityHandle> GetEntityArray(const int32 ArrayIndex)
	{
		return ArrayIndex == 2
			? FloatIntEntities
			: (ArrayIndex == 1
				? IntEntities
				: FloatEntities);
	}

	TArray<FMassEntityHandle> CreateEntitySubset(const int32 NumEntities = NumTestedEntities)
	{
		TArray<FMassEntityHandle> EntitiesSubSet;

		int32 ArrayIndex = 0;
		while (EntitiesSubSet.Num() < NumEntities)
		{
			EntitiesSubSet.AddUnique(GetEntityArray(ArrayIndex)[RandomStream.RandRange(0, EntitiesToCreatePerArchetype - 1)]);
			ArrayIndex = (ArrayIndex + 1) % NumArchetypesUsed;
		}

		return EntitiesSubSet;
	}

	TArray<FMassEntityHandle> GetArraySubset(TConstArrayView<FMassEntityHandle> InArray, int32 NumEntities) const
	{
		TArray<FMassEntityHandle> EntitiesSubSet;
		if (NumEntities >= InArray.Num())
		{
			EntitiesSubSet = InArray;
		}
		else
		{
			while (EntitiesSubSet.Num() < NumEntities)
			{
				EntitiesSubSet.AddUnique(InArray[RandomStream.RandRange(0, InArray.Num() - 1)]);
			}
		}

		return EntitiesSubSet;
	}

	bool CompareCollectionArrays(TConstArrayView<FMassArchetypeEntityCollection> CollectionsA, TConstArrayView<FMassArchetypeEntityCollection> CollectionsB) const
	{
		// note that the order of FMassArchetypeEntityCollection instances in individual arrays might be different,
		// we need to find the matching instances first.
		for (const FMassArchetypeEntityCollection& ArchetypeCollectionA : CollectionsA)
		{
			const FMassArchetypeEntityCollection* ArchetypeCollectionB = CollectionsB.FindByPredicate([&ArchetypeCollectionA](const FMassArchetypeEntityCollection& Element)
			{
				return Element.IsSameArchetype(ArchetypeCollectionA);
			});
			AITEST_NOT_NULL("Matching collection found in the other archetype collection set", ArchetypeCollectionB);
			AITEST_TRUE("Individual archetype collections match", ArchetypeCollectionA.IsSame(*ArchetypeCollectionB));
		}

		return true;
	}

	virtual bool SetUp() override
	{
		if (FEntityTestBase::SetUp())
		{
			EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToCreatePerArchetype, FloatEntities);
			EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToCreatePerArchetype, IntEntities);
			EntityManager->BatchCreateEntities(FloatsIntsArchetype, EntitiesToCreatePerArchetype, FloatIntEntities);

			return true;
		}
		return false;
	}
};

struct FNewlyCreated : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		FEntityCollection EntityCollection;

		AITEST_TRUE("Newly created collection is empty", EntityCollection.IsEmpty());
		AITEST_TRUE("Newly created collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_TRUE("Newly created collection contains no entity handles", EntityCollection.GetEntityHandlesView().IsEmpty());
		AITEST_TRUE("Newly created collection contains no archetype collections", EntityCollection.GetCachedPerArchetypeCollections().IsEmpty());
		AITEST_TRUE("Newly created collection contains no archetype collections after rebuilding them", EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).IsEmpty());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FNewlyCreated, "System.Mass.EntityCollection.Empty");

struct FCreateWithHandle : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		FEntityCollection EntityCollection;
		TArray<FMassEntityHandle> EntitiesSubSet = CreateEntitySubset();

		EntityCollection.AddHandle(EntitiesSubSet[0]);
		AITEST_FALSE("(NOT) Single-handle collection is empty", EntityCollection.IsEmpty());
		AITEST_FALSE("(NOT) Single-handle collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Single-handle collection's number of entity handles", EntityCollection.GetEntityHandlesView().Num(), 1);
		AITEST_EQUAL("Single-handle collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 0);
		AITEST_EQUAL("Single-handle collection's number of updated archetype collections", EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num(), 1);
		AITEST_TRUE("Single-handle collection is up-to-date after updating archetype collections", EntityCollection.IsUpToDate());

		EntityCollection.AddHandle(EntitiesSubSet[1]);
		AITEST_FALSE("(NOT) Two-handles collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Two-handles collection's number of entity handles", EntityCollection.GetEntityHandlesView().Num(), 2);
		AITEST_EQUAL("Two-handles collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 0);

		for (int32 HandleIndex = 2; HandleIndex < EntitiesSubSet.Num(); ++HandleIndex)
		{
			EntityCollection.AddHandle(EntitiesSubSet[HandleIndex]);
			AITEST_EQUAL("Collection's number of entity handles", EntityCollection.GetEntityHandlesView().Num(), HandleIndex + 1);
		}
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 0);
		AITEST_EQUAL("Collection's number of updated archetype collections", EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num(), NumArchetypesUsed);
		AITEST_TRUE("Collection is up-to-date after updating archetype collections", EntityCollection.IsUpToDate());

		// now we'll verify that the order in which handles are added is irrelevant
		FEntityCollection SecondEntityCollection;
		for (int32 HandleIndex = EntitiesSubSet.Num() - 1; HandleIndex >= 0; --HandleIndex)
		{
			SecondEntityCollection.AddHandle(EntitiesSubSet[HandleIndex]);
		}

		TConstArrayView<FMassArchetypeEntityCollection> ArchetypeCollections = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get());
		TConstArrayView<FMassArchetypeEntityCollection> SecondArchetypeCollections = SecondEntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get());

		return CompareCollectionArrays(ArchetypeCollections, SecondArchetypeCollections);
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreateWithHandle, "System.Mass.EntityCollection.PopulateWithIndividualHandles");

struct FCreateWithHandleArrays : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		FEntityCollection EntityCollection;
		TArray<FMassEntityHandle> EntitiesSubSet = CreateEntitySubset();
		check(EntitiesSubSet.Num() >= 5);
		TArray<TConstArrayView<FMassEntityHandle>> SubViews = {
			MakeArrayView(&EntitiesSubSet[0], EntitiesSubSet.Num() / 2)
			, MakeArrayView(&EntitiesSubSet[EntitiesSubSet.Num() / 2], 2)
			, MakeArrayView(&EntitiesSubSet[EntitiesSubSet.Num() / 2 + 2], EntitiesSubSet.Num() / 2 - 2)
		};
		
		EntityCollection.AppendHandles(SubViews[0]);
		AITEST_FALSE("(NOT) Collection is empty", EntityCollection.IsEmpty());
		AITEST_FALSE("(NOT) Collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Collection's number of entity handles from first slice", EntityCollection.GetEntityHandlesView().Num(), SubViews[0].Num());
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 0);
		const int32 NumArchetypesInFirstSlice = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num();
		AITEST_TRUE("Collection's number of updated archetype collections from first slice", NumArchetypesInFirstSlice > 0 && NumArchetypesInFirstSlice <= NumArchetypesUsed);
		AITEST_TRUE("Collection is up-to-date after updating archetype collections", EntityCollection.IsUpToDate());

		EntityCollection.AppendHandles(SubViews[1]);
		AITEST_FALSE("(NOT) Two-slice collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Two-slice collection's number of entities", EntityCollection.GetEntityHandlesView().Num(), SubViews[0].Num() + SubViews[1].Num());
		const int32 NumArchetypesInTwoSlices = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num();
		AITEST_TRUE("Collection's number of updated archetype collections from two slices", NumArchetypesInTwoSlices >= NumArchetypesInFirstSlice);

		EntityCollection.AppendHandles(SubViews[2]);
		AITEST_EQUAL("Two-slice collection's number of entities", EntityCollection.GetEntityHandlesView().Num(), EntitiesSubSet.Num());
		const int32 TotalNumArchetypes = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get()).Num();
		AITEST_EQUAL("Collection's number of updated archetype collections from all entities", TotalNumArchetypes, NumArchetypesUsed);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreateWithHandleArrays, "System.Mass.EntityCollection.PopulateWithHandleArrays");

struct FCreateWithCollections : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> EntitiesSubSet = CreateEntitySubset();
		TArray<FMassArchetypeEntityCollection> ArchetypeCollections;
		Utils::CreateEntityCollections(*EntityManager.Get(), EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates, ArchetypeCollections);
		check(ArchetypeCollections.Num() == NumArchetypesUsed);
		static_assert(NumArchetypesUsed >= 3);

		FEntityCollection EntityCollection;
		EntityCollection.AppendCollection(ArchetypeCollections[0]);
		AITEST_FALSE("(NOT) Collection is empty", EntityCollection.IsEmpty());
		AITEST_TRUE("Collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 1);

		EntityCollection.AppendCollection(MoveTemp(ArchetypeCollections[1]));
		AITEST_TRUE("Collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 2);

		EntityCollection.AppendCollection(ArchetypeCollections[2]);
		AITEST_TRUE("Collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 3);

		AITEST_EQUAL("Collection's number of entities", EntityCollection.GetEntityHandlesView().Num(), EntitiesSubSet.Num());

		TArray<FMassEntityHandle> ExportedHandles;
		ExportedHandles.Append(EntityCollection.GetEntityHandlesView());
		ExportedHandles.Sort();
		EntitiesSubSet.Sort();
		AITEST_TRUE("Collection's entity handles match expectation", Algo::Compare(ExportedHandles, EntitiesSubSet));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreateWithCollections, "System.Mass.EntityCollection.PopulateWithCollections");

struct FCreateWithOutdatedCollections : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 NumEntities = 10;
		TArray<FMassEntityHandle> EntitiesSubSet = GetArraySubset(FloatEntities, NumEntities);
		FMassArchetypeEntityCollection InitialCollection = FMassArchetypeEntityCollection(FloatsArchetype, EntitiesSubSet, FMassArchetypeEntityCollection::NoDuplicates);
		AITEST_TRUE("Initially the created archetype collection is up to date", InitialCollection.IsUpToDate());

		// we now move the entities to another archetype, to force InitialCollection's invalidation
		EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&InitialCollection, 1), FMassFragmentBitSet(FTestFragment_Int::StaticStruct()), {});

		AITEST_FALSE("(NOT) After entities are moved to another archetype the InitialCollection is up to date", InitialCollection.IsUpToDate());
		{
			MASS_SCOPED_ENSURE_TEST("The entity collection is out of date", 1);
			FEntityCollection EntityCollection(InitialCollection);
			AITEST_TRUE("The EntityCollection is still empty", EntityCollection.IsEmpty());
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreateWithOutdatedCollections, "System.Mass.EntityCollection.PopulateWithOutdatedCollections");

struct FCreateWithCollectionsHandlesPairs : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		const int32 SubEntitiesPerArchetype = EntitiesToCreatePerArchetype / 3;
		TArray<TArray<FMassEntityHandle>> PerArchetypeEntitiesSubSet = {
			GetArraySubset(FloatEntities, SubEntitiesPerArchetype )
			, GetArraySubset(IntEntities, SubEntitiesPerArchetype )
			, GetArraySubset(FloatIntEntities, SubEntitiesPerArchetype )
		};

		TArray<FMassArchetypeEntityCollection> ArchetypeCollections = {
			FMassArchetypeEntityCollection(FloatsArchetype, PerArchetypeEntitiesSubSet[0], FMassArchetypeEntityCollection::NoDuplicates)
			, FMassArchetypeEntityCollection(IntsArchetype, PerArchetypeEntitiesSubSet[1], FMassArchetypeEntityCollection::NoDuplicates)
			, FMassArchetypeEntityCollection(FloatsIntsArchetype, PerArchetypeEntitiesSubSet[2], FMassArchetypeEntityCollection::NoDuplicates)
		};

		FEntityCollection EntityCollection;
		EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[0], MoveTemp(ArchetypeCollections[0]));
		AITEST_FALSE("(NOT) Collection is empty", EntityCollection.IsEmpty());
		AITEST_TRUE("Collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 1);
		AITEST_EQUAL("Collection's number of stored handles", EntityCollection.GetEntityHandlesView().Num(), SubEntitiesPerArchetype);

		EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[1], MoveTemp(ArchetypeCollections[1]));
		AITEST_TRUE("Collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 2);
		AITEST_EQUAL("Collection's number of stored entities after second operation", EntityCollection.GetEntityHandlesView().Num(), SubEntitiesPerArchetype * 2);

		EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[2], MoveTemp(ArchetypeCollections[2]));
		AITEST_TRUE("Collection is up-to-date", EntityCollection.IsUpToDate());
		AITEST_EQUAL("Collection's number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 3);
		AITEST_EQUAL("Collection's number of stored entities after last operation", EntityCollection.GetEntityHandlesView().Num(), SubEntitiesPerArchetype * 3);

		TArray<FMassArchetypeEntityCollection> CachedCollections;
		CachedCollections.Append(EntityCollection.GetCachedPerArchetypeCollections());
		AITEST_TRUE("Cached collections are the same as updated"
			, CompareCollectionArrays(
				CachedCollections
				, EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreateWithCollectionsHandlesPairs, "System.Mass.EntityCollection.PopulateWithCollectionsHandlesPairs");

struct FCreateWithDuplicates : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 DuplicatesCount = 5;
		FEntityCollection EntityCollection;
		FMassEntityHandle EntityHandle = FloatEntities[RandomStream.RandRange(0, FloatEntities.Num() - 1)];

		for (int32 Counter = 0; Counter < DuplicatesCount; ++Counter)
		{
			EntityCollection.AddHandle(EntityHandle);
		}

		AITEST_EQUAL("Collection's number of stored entity handles", EntityCollection.GetEntityHandlesView().Num(), DuplicatesCount);
		AITEST_FALSE("(NOT) Collection is up-to-date", EntityCollection.IsUpToDate());
		TConstArrayView<FMassArchetypeEntityCollection> Collections = EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager);
		AITEST_EQUAL("Number of up-to-date archetype collections", Collections.Num(), 1);
#if WITH_MASSENTITY_DEBUG
		AITEST_EQUAL("Number of entities in archetype collections", Collections[0].DebugCountEntities(), 1);
#endif // WITH_MASSENTITY_DEBUG

		AITEST_EQUAL("Collection's number of stored entity handles after getting updated collections", EntityCollection.GetEntityHandlesView().Num(), DuplicatesCount);
		AITEST_TRUE("Duplicates identified and removed", EntityCollection.UpdateAndRemoveDuplicates(*EntityManager));
		AITEST_EQUAL("Number of entities after removing duplicates", EntityCollection.GetEntityHandlesView().Num(), 1);
		AITEST_EQUAL("Number of cached archetype collections", EntityCollection.GetCachedPerArchetypeCollections().Num(), 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreateWithDuplicates, "System.Mass.EntityCollection.PopulateWithDuplicates");

struct FCreationMethodEquivalency : FEntityCollectionTestBase
{
	virtual bool InstantTest() override
	{
		const int32 SubEntitiesPerArchetype = EntitiesToCreatePerArchetype / 3;
		TArray<TArray<FMassEntityHandle>> PerArchetypeEntitiesSubSet = {
			GetArraySubset(FloatEntities, SubEntitiesPerArchetype )
			, GetArraySubset(IntEntities, SubEntitiesPerArchetype )
			, GetArraySubset(FloatIntEntities, SubEntitiesPerArchetype )
		};

		TArray<FMassEntityHandle> EntitiesSubSet = PerArchetypeEntitiesSubSet[0];
		EntitiesSubSet.Append(PerArchetypeEntitiesSubSet[1]);
		EntitiesSubSet.Append(PerArchetypeEntitiesSubSet[2]);

		FEntityCollection EntityCollectionFromArray(EntitiesSubSet);
		FEntityCollection EntityCollectionFromHandles;
		for (const FMassEntityHandle& EntityHandle : EntitiesSubSet)
		{
			EntityCollectionFromHandles.AddHandle(EntityHandle);
		}
		AITEST_TRUE("Collections created from an array of handles vs individual handles"
			, CompareCollectionArrays(
				EntityCollectionFromArray.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
				, EntityCollectionFromHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

		// the following collection is constructed with a mix of arrays and individual handles, starting with a handle
		FEntityCollection EntityCollectionFromHandlesAndArrays;
		{
			EntityCollectionFromHandlesAndArrays.AddHandle(EntitiesSubSet[0]);
			EntityCollectionFromHandlesAndArrays.AppendHandles(MakeArrayView(&EntitiesSubSet[1], (EntitiesSubSet.Num()) / 2));
			EntityCollectionFromHandlesAndArrays.AddHandle(EntitiesSubSet[EntityCollectionFromHandlesAndArrays.GetEntityHandlesView().Num()]);
			const int32 NumHandlesStoredAlready = EntityCollectionFromHandlesAndArrays.GetEntityHandlesView().Num();
			EntityCollectionFromHandlesAndArrays.AppendHandles(MakeArrayView(&EntitiesSubSet[NumHandlesStoredAlready], EntitiesSubSet.Num() - NumHandlesStoredAlready));
		}
		FEntityCollection EntityCollectionFromArraysAndHandles;
		{
			EntityCollectionFromArraysAndHandles.AppendHandles(MakeArrayView(&EntitiesSubSet[0], (EntitiesSubSet.Num()) / 2));
			EntityCollectionFromArraysAndHandles.AddHandle(EntitiesSubSet[EntityCollectionFromArraysAndHandles.GetEntityHandlesView().Num()]);
			const int32 NumHandlesStoredAlready = EntityCollectionFromArraysAndHandles.GetEntityHandlesView().Num();
			EntityCollectionFromArraysAndHandles.AppendHandles(MakeArrayView(&EntitiesSubSet[NumHandlesStoredAlready], EntitiesSubSet.Num() - NumHandlesStoredAlready));
		}

		AITEST_TRUE("Collections created with a mix of handles and arrays"
			, CompareCollectionArrays(
				EntityCollectionFromHandlesAndArrays.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
				, EntityCollectionFromArraysAndHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

		AITEST_TRUE("Collections created with a mix of handles and arrays vs heterogeneous approaches"
			, CompareCollectionArrays(
				EntityCollectionFromHandlesAndArrays.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
				, EntityCollectionFromHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

		TArray<FMassArchetypeEntityCollection> ArchetypeCollections = {
			FMassArchetypeEntityCollection(FloatsArchetype, PerArchetypeEntitiesSubSet[0], FMassArchetypeEntityCollection::NoDuplicates)
			, FMassArchetypeEntityCollection(IntsArchetype, PerArchetypeEntitiesSubSet[1], FMassArchetypeEntityCollection::NoDuplicates)
			, FMassArchetypeEntityCollection(FloatsIntsArchetype, PerArchetypeEntitiesSubSet[2], FMassArchetypeEntityCollection::NoDuplicates)
		};

		FEntityCollection EntityCollection(PerArchetypeEntitiesSubSet[0], MoveTemp(ArchetypeCollections[0]));
		EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[1], MoveTemp(ArchetypeCollections[1]));
		EntityCollection.AppendHandles(PerArchetypeEntitiesSubSet[2], MoveTemp(ArchetypeCollections[2]));

		AITEST_TRUE("Collection created with handles-and-archetype-collections vs handles-only"
			, CompareCollectionArrays(
				EntityCollection.GetUpToDatePerArchetypeCollections(*EntityManager.Get())
				, EntityCollectionFromHandles.GetUpToDatePerArchetypeCollections(*EntityManager.Get())));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreationMethodEquivalency, "System.Mass.EntityCollection.MethodEquivalency");

}

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
