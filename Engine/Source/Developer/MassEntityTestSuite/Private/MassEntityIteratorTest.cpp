// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"
#include "Misc/SpinLock.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP
#if WITH_MASSENTITY_DEBUG

namespace UE::Mass::Test::EntityIterator
{

struct FIterator_IndexParity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		float NumChunksToPopulate = 2.3f;

		const int32 NumEntities = static_cast<int32>(NumChunksToPopulate * static_cast<float>(EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype)));
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumEntities, EntitiesCreated);

		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);

		TArray<FMassEntityHandle> EntitiesIndexed;
		TArray<FMassEntityHandle> EntitiesIterated;

		Processor->ForEachEntityChunkExecutionFunction = [&EntitiesIndexed](const FMassExecutionContext& Context)
		{
			for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); ++EntityIndex)
			{
				EntitiesIndexed.Add(Context.GetEntity(EntityIndex));
			}
		};
		Processor->TestExecute(EntityManager);

		Processor->ForEachEntityChunkExecutionFunction = [&EntitiesIterated](FMassExecutionContext& Context)
		{
			for (FMassExecutionContext::FEntityIterator EntityIterator = Context.CreateEntityIterator(); EntityIterator; ++EntityIterator)
			{
				EntitiesIterated.Add(Context.GetEntity(EntityIterator));
			}
		};
		Processor->TestExecute(EntityManager);

		AITEST_TRUE("Index-based loop processes all entities", EntitiesCreated.Num() == EntitiesIndexed.Num());
		AITEST_TRUE("Iterator-based loop processes all entities", EntitiesCreated.Num() == EntitiesIterated.Num());
		AITEST_TRUE("Index-based and iterator-based processing produce same results "
			, FMemory::Memcmp(EntitiesIterated.GetData(), EntitiesIndexed.GetData(), EntitiesIndexed.Num()) == 0);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FIterator_IndexParity, "System.Mass.Entity.Iterator.Parity");

struct FIterator_ParallelFor : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		float NumChunksToPopulate = 21.3f;

		const int32 NumEntities = static_cast<int32>(NumChunksToPopulate * static_cast<float>(EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype)));
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumEntities, EntitiesCreated);

		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
		
		TArray<FMassEntityHandle> EntitiesSync;
		TArray<FMassEntityHandle> EntitiesAsync;

		Processor->ForEachEntityChunkExecutionFunction = [&EntitiesSync](FMassExecutionContext& Context)
		{
			for (FMassExecutionContext::FEntityIterator EntityIterator = Context.CreateEntityIterator(); EntityIterator; ++EntityIterator)
			{
				EntitiesSync.Add(Context.GetEntity(EntityIterator));
			}
		};
		Processor->TestExecute(EntityManager);

		FSpinLock Lock;
		Processor->ForEachEntityChunkExecutionFunction = [&EntitiesAsync, &Lock](FMassExecutionContext& Context)
		{
			TArray<FMassEntityHandle> EntitiesAsyncLocal;
			for (FMassExecutionContext::FEntityIterator EntityIterator = Context.CreateEntityIterator(); EntityIterator; ++EntityIterator)
			{
				EntitiesAsyncLocal.Add(Context.GetEntity(EntityIterator));
			}

			Lock.Lock();
			EntitiesAsync.Append(EntitiesAsyncLocal);
			Lock.Unlock();
		};
		Processor->SetUseParallelForEachEntityChunk(true);
		Processor->TestExecute(EntityManager);

		EntitiesSync.Sort();
		EntitiesAsync.Sort();

		AITEST_TRUE("Index-based loop processes all entities", EntitiesCreated.Num() == EntitiesSync.Num());
		AITEST_TRUE("Iterator-based loop processes all entities", EntitiesCreated.Num() == EntitiesAsync.Num());
		AITEST_TRUE("Index-based and iterator-based processing produce same results "
			, FMemory::Memcmp(EntitiesAsync.GetData(), EntitiesSync.GetData(), EntitiesSync.Num()) == 0);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FIterator_ParallelFor, "System.Mass.Entity.Iterator.ParallelFor");

struct FIterator_QuerylessIterator : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassExecutionContext LocalContext(*EntityManager.Get());
		{
			MASS_SCOPED_ENSURE_TEST("no entity query is being executed", 1);
			FMassExecutionContext::FEntityIterator FailedIterator = LocalContext.CreateEntityIterator();

			AITEST_FALSE("(Not) Created iterator is valid", bool(FailedIterator));

			int32 NumIterations = 0;
			for (; FailedIterator; ++FailedIterator)
			{
				++NumIterations;
			}
			AITEST_EQUAL("Number of iterations with an invalid iterator", NumIterations, 0);
		}
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FIterator_QuerylessIterator, "System.Mass.Entity.Iterator.Queryless");

struct FIterator_ProcessorlessIterator : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassExecutionContext LocalContext(*EntityManager.Get());
		{
			MASS_SCOPED_ENSURE_TEST("no entity query is being executed", 1);
			FMassExecutionContext::FEntityIterator FailedIterator = LocalContext.CreateEntityIterator();

			AITEST_FALSE("(Not) Created iterator is valid", bool(FailedIterator));

			int32 NumIterations = 0;
			for (; FailedIterator; ++FailedIterator)
			{
				++NumIterations;
			}
			AITEST_EQUAL("Number of iterations with an invalid iterator", NumIterations, 0);
		}
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FIterator_ProcessorlessIterator, "System.Mass.Entity.Iterator.Processorless");

} // UE::Mass::Test

#endif // WITH_MASSENTITY_DEBUG
UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
