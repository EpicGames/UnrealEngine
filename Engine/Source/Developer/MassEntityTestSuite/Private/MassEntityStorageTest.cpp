// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"

// not point in "parity testing" if there's only sequential Mass storage available
#if WITH_MASS_CONCURRENT_RESERVE

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassEntityStorageTest
{

enum class EEntityManagerMode : int32
{
	Sequential,
	Concurrent
};

struct FEntityStorageTestBase : FAITestBase
{
	enum EEntityOperation
	{
		Operation_IndividualReserve,
		Operation_BatchReserve,
		Operation_IndividualDestroy,
		Operation_BatchDestroy,

		OperationsCount,
	};

	TArray<TSharedPtr<FMassEntityManager>> EntityManagers;
	TArray<EEntityManagerMode> ManagersToCreate;
	const uint32 MaxConcurrentEntitiesPerPage = 0;
	const int32 TotalNumToReserve = 0;
	const int32 OperationsNumLimit = OperationsCount * 5;
	int32 NumToReserveInOneIteration = 0;
	int32 NumToReleaseInOneIteration = 0;
	FRandomStream RandomStream;

	FEntityStorageTestBase()
		// default setup: one sequential and one concurrent
		: ManagersToCreate({ EEntityManagerMode::Sequential, EEntityManagerMode::Concurrent })
		, MaxConcurrentEntitiesPerPage(FMassEntityManager_InitParams_Concurrent().MaxEntitiesPerPage)
		// note that MaxConcurrentEntitiesPerPage doesn't really have any meaning for Sequential storage, but it's
		// as good of a value to use for this test as any other
		, TotalNumToReserve(MaxConcurrentEntitiesPerPage * 3 / 2)
		, RandomStream(1)
	{
		NumToReserveInOneIteration = TotalNumToReserve / 10;
		NumToReleaseInOneIteration = TotalNumToReserve / 12;
	}

	void PerformOperation(const EEntityOperation CurrentOperation, TSharedPtr<FMassEntityManager>& EntityManager, TArray<FMassEntityHandle>& EntitiesReserved)
	{
		switch (CurrentOperation)
		{
		case Operation_IndividualReserve:
			for (int32 Counter = 0; Counter < NumToReserveInOneIteration; ++Counter)
			{
				EntitiesReserved.Add(EntityManager->ReserveEntity());
			}
			break;

		case Operation_BatchReserve:
			EntityManager->BatchReserveEntities(NumToReserveInOneIteration, EntitiesReserved);
			break;

		case Operation_IndividualDestroy:
		{
			const int32 NumToRelease = FMath::Min(NumToReleaseInOneIteration, EntitiesReserved.Num());
			for (int32 Counter = 0; Counter < NumToRelease; ++Counter)
			{
				const int32 Index = RandomStream.RandRange(0, EntitiesReserved.Num() - 1);
				EntityManager->ReleaseReservedEntity(EntitiesReserved[Index]);
				EntitiesReserved.RemoveAtSwap(Index, EAllowShrinking::No);
			}
		}
		break;

		case Operation_BatchDestroy:
		{
			const int32 NumToRelease = FMath::Min(NumToReleaseInOneIteration, EntitiesReserved.Num());
			TArray<FMassEntityHandle> EntitiesToDestroy;
			EntitiesToDestroy.Reserve(NumToRelease);
			for (int32 Counter = 0; Counter < NumToRelease; ++Counter)
			{
				const int32 Index = RandomStream.RandRange(0, EntitiesReserved.Num() - 1);
				EntitiesToDestroy.Add(EntitiesReserved[Index]);
				EntitiesReserved.RemoveAtSwap(Index, EAllowShrinking::No);
			}
			EntityManager->BatchDestroyEntities(EntitiesToDestroy);
		}
		break;
		}
	}

	virtual bool SetUp() override
	{
		int32 Index = 0;
		for (const EEntityManagerMode Mode : ManagersToCreate)
		{
			TSharedPtr<FMassEntityManager> LocalEntityManager = MakeShareable(new FMassEntityManager());
			LocalEntityManager->SetDebugName(FString::Printf(TEXT("TestEntityManager_%d"), Index));
			++Index;

			FMassEntityManagerStorageInitParams InitializationParams;
			if (Mode == EEntityManagerMode::Sequential)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				InitializationParams.Emplace<FMassEntityManager_InitParams_SingleThreaded>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			else // Mode == EEntityManagerMode::Concurrent
			{
				InitializationParams.Emplace<FMassEntityManager_InitParams_Concurrent>();
				InitializationParams.Get<FMassEntityManager_InitParams_Concurrent>().MaxEntitiesPerPage = MaxConcurrentEntitiesPerPage;
			}
			LocalEntityManager->Initialize(InitializationParams);

			EntityManagers.Add(LocalEntityManager);
		}
		
		return true;
	}

	/** 
	 * @return whether there are no duplicates. The return value matches the results produced by AITEST macros, 
	 * thus the seemingly reversed function's meaning
	 */
	bool ValidateUniqueAndValidEntities(TSharedPtr<FMassEntityManager>& EntityManager, TConstArrayView<FMassEntityHandle> ContainerToTest) const
	{
		ContainerToTest.Sort();
		for (int32 Index = ContainerToTest.Num() - 1; Index >= 0; --Index)
		{
			if (Index > 0)
			{
				AITEST_FALSE("Checking for duplicates", ContainerToTest[Index].Index == ContainerToTest[Index - 1].Index);
			}
			AITEST_TRUE("We expect every handle to be valid", EntityManager->IsEntityValid(ContainerToTest[Index]));
			AITEST_FALSE("None of the gathered entities is expected to have been built", EntityManager->IsEntityBuilt(ContainerToTest[Index]));
		}

		return true;
	}
};

struct FEntityStorageTest_SingleEntityParity : FEntityStorageTestBase
{	
	virtual bool InstantTest() override
	{
		//AITEST_TRUE("The reserved entity should be a valid entity", EntityManager->IsEntityValid(ReservedEntity));
		FMassEntityHandle ReservedEntitySequential = EntityManagers[int(EEntityManagerMode::Sequential)]->ReserveEntity();
		FMassEntityHandle ReservedEntityConcurrent = EntityManagers[int(EEntityManagerMode::Concurrent)]->ReserveEntity();
		AITEST_EQUAL("The reserved entities are expected to be the same, regardless of the storage type", ReservedEntitySequential, ReservedEntityConcurrent);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_SingleEntityParity, "System.Mass.Storage.SingleEntityParity");

/** This test base compares results of multiple calls to ReserveEntity and a single call to BatchReserveEntities */
struct FEntityStorageTest_ParityBase : FEntityStorageTestBase
{
	virtual bool InstantTest() override
	{
		const int32 NumToReserve = MaxConcurrentEntitiesPerPage * 3 / 2;

		TArray<FMassEntityHandle> EntitiesIndividual;
		for (int32 Index = 0; Index < NumToReserve; ++Index)
		{
			EntitiesIndividual.Add(EntityManagers[0]->ReserveEntity());
		}
		AITEST_EQUAL("The number of individually reserved entities should match the requested count", EntitiesIndividual.Num(), NumToReserve);

		TArray<FMassEntityHandle> EntitiesBatch;
		EntityManagers[1]->BatchReserveEntities(NumToReserve, EntitiesBatch);
		AITEST_EQUAL("The number of batch-reserved entities should match the requested count", EntitiesBatch.Num(), NumToReserve);

		for (int Index = 0; Index < NumToReserve; ++Index)
		{
			AITEST_EQUAL("The reserved entities are expected to be the same, regardless of the storage type", EntitiesIndividual[Index].Index, EntitiesBatch[Index].Index);
		}

		return true;
	}
};

struct FEntityStorageTest_ParitySequential : FEntityStorageTest_ParityBase
{
	FEntityStorageTest_ParitySequential()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Sequential, EEntityManagerMode::Sequential });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_ParitySequential, "System.Mass.Storage.Sequential.BatchParity");

struct FEntityStorageTest_ParityConcurrent : FEntityStorageTest_ParityBase
{
	FEntityStorageTest_ParityConcurrent()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Concurrent, EEntityManagerMode::Concurrent });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_ParityConcurrent, "System.Mass.Storage.Concurrent.BatchParity");

struct FEntityStorageTest_FreeListBase : FEntityStorageTestBase
{
	virtual bool InstantTest() override
	{
		// note that MaxConcurrentEntitiesPerPage doesn't really have any meaning for Sequential storage, but it's
		// as good of a value to use for this test as any other
		const int32 NumToReserve = MaxConcurrentEntitiesPerPage * 3 / 2;

		TArray<FMassEntityHandle> EntitiesBaseline;
		EntityManagers[0]->BatchReserveEntities(NumToReserve, EntitiesBaseline);

		TArray<FMassEntityHandle> EntitiesTestedBatch;
		// batch-reserving and batch-removing
		{
			EntityManagers[1]->BatchReserveEntities(NumToReserve, EntitiesTestedBatch);

			TArray<FMassEntityHandle> EntitiesToModify;
			EntitiesToModify.Reserve(EntitiesTestedBatch.Num() / 2);
			for (int32 Index = EntitiesTestedBatch.Num() - 1; Index >= 0; --Index)
			{
				if (EntitiesTestedBatch[Index].Index % 2)
				{
					EntitiesToModify.Add(EntitiesTestedBatch[Index]);
					EntitiesTestedBatch.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}
			const int32 EntitiesRemovedCount = EntitiesToModify.Num();
			EntityManagers[1]->BatchDestroyEntities(EntitiesToModify);
			EntitiesToModify.Reset();
			EntityManagers[1]->BatchReserveEntities(EntitiesRemovedCount, EntitiesToModify);
			AITEST_EQUAL("Reserving after removing should produce the expected number of entity handles", EntitiesToModify.Num(), EntitiesRemovedCount);

			EntitiesTestedBatch.Append(EntitiesToModify);
			AITEST_EQUAL("Entity handle arrays are expected to be of the same size", EntitiesTestedBatch.Num(), EntitiesBaseline.Num());
		}

		// batch-reserving and removing one-by-one
		TArray<FMassEntityHandle> EntitiesTestedIndividual;
		{
			EntityManagers[2]->BatchReserveEntities(NumToReserve, EntitiesTestedIndividual);

			for (int32 Index = EntitiesTestedIndividual.Num() - 1; Index >= 0; --Index)
			{
				if (RandomStream.FRand() >= 0.5)
				{
					EntityManagers[2]->ReleaseReservedEntity(EntitiesTestedIndividual[Index]);
					EntitiesTestedIndividual.RemoveAtSwap(Index, EAllowShrinking::No);
				}
			}

			EntityManagers[2]->BatchReserveEntities(NumToReserve - EntitiesTestedIndividual.Num(), EntitiesTestedIndividual);
			AITEST_EQUAL("Total number of entities reserved should be back to initial number", EntitiesTestedIndividual.Num(), NumToReserve);
		}

		EntitiesTestedIndividual.Sort();
		EntitiesTestedBatch.Sort();
		EntitiesBaseline.Sort();
		for (int32 Index = 0; Index < EntitiesTestedBatch.Num(); ++Index)
		{ 
			AITEST_EQUAL("All entries are expected to be equivalent A", EntitiesTestedBatch[Index].Index, EntitiesBaseline[Index].Index);
			AITEST_EQUAL("All entries are expected to be equivalent B", EntitiesTestedIndividual[Index].Index, EntitiesBaseline[Index].Index);
		}

		return true;
	}
};

struct FEntityStorageTest_FreeListSequential : FEntityStorageTest_FreeListBase
{
	FEntityStorageTest_FreeListSequential()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Sequential, EEntityManagerMode::Sequential, EEntityManagerMode::Sequential });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_FreeListSequential, "System.Mass.Storage.Sequential.FreeList");

struct FEntityStorageTest_FreeListConcurrent : FEntityStorageTest_FreeListBase
{
	FEntityStorageTest_FreeListConcurrent()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Concurrent, EEntityManagerMode::Concurrent, EEntityManagerMode::Concurrent });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_FreeListConcurrent, "System.Mass.Storage.Concurrent.FreeList");

//-----------------------------------------------------------------------------
// Special case that failed during development. 
// The free list was not being updated properly after removing entities so a sequence of add-remove-add-add 
// caused the last `add` to overlap with the last-but-one `add`
//-----------------------------------------------------------------------------
struct FEntityStorageTest_MultiReAddingBase : FEntityStorageTestBase
{
	virtual bool InstantTest() override
	{
		TSharedPtr<FMassEntityManager> EntityManager = EntityManagers[0];
		TArray<FMassEntityHandle> EntitiesReserved;

		PerformOperation(Operation_BatchReserve, EntityManager, EntitiesReserved);
		PerformOperation(Operation_BatchDestroy, EntityManager, EntitiesReserved);
		PerformOperation(Operation_BatchReserve, EntityManager, EntitiesReserved);
		AITEST_TRUE("Testing for duplicates after first readding", ValidateUniqueAndValidEntities(EntityManager, EntitiesReserved))

		PerformOperation(Operation_BatchReserve, EntityManager, EntitiesReserved);
		AITEST_TRUE("Testing for duplicates after n+1 readding", ValidateUniqueAndValidEntities(EntityManager, EntitiesReserved))

		return true;
	}
};

struct FEntityStorageTest_MultiReAddingSequential : FEntityStorageTest_MultiReAddingBase
{
	FEntityStorageTest_MultiReAddingSequential()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Sequential });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_MultiReAddingSequential, "System.Mass.Storage.Sequential.AddRemoveLoop");

struct FEntityStorageTest_MultiReAddingConcurrent : FEntityStorageTest_MultiReAddingBase
{
	FEntityStorageTest_MultiReAddingConcurrent()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Concurrent });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_MultiReAddingConcurrent, "System.Mass.Storage.Concurrent.AddRemoveLoop");

//-----------------------------------------------------------------------------
// catch-all test
//-----------------------------------------------------------------------------
struct FEntityStorageTest_MixedOperationsBase : FEntityStorageTestBase
{
	virtual bool InstantTest() override
	{
		TSharedPtr<FMassEntityManager> EntityManager = EntityManagers[0];

		TArray<int32> OperationsPerformed;
		TArray<FMassEntityHandle> EntitiesReserved;

		EEntityOperation CurrentOperation = Operation_BatchReserve;
		while (EntitiesReserved.Num() < TotalNumToReserve && OperationsPerformed.Num() < OperationsNumLimit)
		{
			PerformOperation(CurrentOperation, EntityManager, EntitiesReserved);
			const bool bValidColleciton = ValidateUniqueAndValidEntities(EntityManager, EntitiesReserved);
			AITEST_TRUE("Testing for duplicates", bValidColleciton);
			
			OperationsPerformed.Add(CurrentOperation);
			CurrentOperation = EEntityOperation(RandomStream.RandRange(0, 3));
		}

		return true;
	}
};

struct FEntityStorageTest_MixedOperationsSequential : FEntityStorageTest_MixedOperationsBase
{
	FEntityStorageTest_MixedOperationsSequential()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Sequential });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_MixedOperationsSequential, "System.Mass.Storage.Sequential.MixedOperations");

struct FEntityStorageTest_MixedOperationsConcurrent : FEntityStorageTest_MixedOperationsBase
{
	FEntityStorageTest_MixedOperationsConcurrent()
	{
		ManagersToCreate = TArray<EEntityManagerMode>({ EEntityManagerMode::Concurrent });
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_MixedOperationsConcurrent, "System.Mass.Storage.Concurrent.MixedOperations");

#if WITH_MASSENTITY_DEBUG
struct FEntityStorageTest_ConcurrentDataLayoutAssumptions : FAITestBase
{
	virtual bool InstantTest() override
	{
		const bool bAssumptionsValid = UE::Mass::FConcurrentEntityStorage::DebugAssumptionsSelfTest();
		AITEST_TRUE("Testing assumptions", bAssumptionsValid);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityStorageTest_ConcurrentDataLayoutAssumptions, "System.Mass.Storage.Concurrent.DataLayoutAssumptions");
#endif // WITH_MASSENTITY_DEBUG
} // FMassEntityTestTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE

#endif // WITH_MASS_CONCURRENT_RESERVE