// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "MassProcessingTypes.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassEntityTest
{
#if WITH_MASSENTITY_DEBUG
struct FEntityTest_ArchetypeCreation : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		AITEST_TRUE("Floats archetype should have been created", FloatsArchetype.IsValid());
		AITEST_TRUE("Floats archetype should have been created", FloatsArchetype.IsValid());
		AITEST_TRUE("Ints archetype should have been created", IntsArchetype.IsValid());

		TArray<const UScriptStruct*> FragmentsList;
		EntityManager->DebugGetArchetypeFragmentTypes(FloatsArchetype, FragmentsList);
		AITEST_EQUAL("Floats archetype should contain just a single fragment", FragmentsList.Num(), 1);
		AITEST_EQUAL("Floats archetype\'s lone fragment should be of Float fragment type", FragmentsList[0], FTestFragment_Float::StaticStruct());

		FragmentsList.Reset();
		EntityManager->DebugGetArchetypeFragmentTypes(IntsArchetype, FragmentsList);
		AITEST_EQUAL("Ints archetype should contain just a single fragment", FragmentsList.Num(), 1);
		AITEST_EQUAL("Ints archetype\'s lone fragment should be of Ints fragment type", FragmentsList[0], FTestFragment_Int::StaticStruct());

		FragmentsList.Reset();
		EntityManager->DebugGetArchetypeFragmentTypes(FloatsIntsArchetype, FragmentsList);
		AITEST_EQUAL("FloatsInts archetype should contain exactly two fragments", FragmentsList.Num(), 2);
		AITEST_TRUE("FloatsInts archetype\'s should contain both expected fragment types", FragmentsList.Find(FTestFragment_Int::StaticStruct()) != INDEX_NONE && FragmentsList.Find(FTestFragment_Float::StaticStruct()) != INDEX_NONE);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_ArchetypeCreation, "System.Mass.Entity.AchetypesCreation");


struct FEntityTest_ArchetypeEquivalence : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		TArray<const UScriptStruct*> FragmentsA = { FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() };
		TArray<const UScriptStruct*> FragmentsB = { FTestFragment_Int::StaticStruct(), FTestFragment_Float::StaticStruct() };
		const FMassArchetypeHandle ArchetypeA = EntityManager->CreateArchetype(FragmentsA);
		const FMassArchetypeHandle ArchetypeB = EntityManager->CreateArchetype(FragmentsB);

		AITEST_EQUAL("Archetype creation is expected to be independent of fragments ordering", ArchetypeA, ArchetypeB);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_ArchetypeEquivalence, "System.Mass.Entity.AchetypeEquivalance");


struct FEntityTest_MultipleEntitiesCreation : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		int32 Counts[] = { 10, 100, 1000 };
		int32 TotalCreatedCount = 0;
		FMassArchetypeHandle Archetypes[] = { FloatsArchetype, IntsArchetype, FloatsIntsArchetype };

		for (int ArchetypeIndex = 0; ArchetypeIndex < (sizeof(Archetypes) / sizeof(FMassArchetypeHandle)); ++ArchetypeIndex)
		{
			for (int i = 0; i < Counts[ArchetypeIndex]; ++i)
			{
				EntityManager->CreateEntity(Archetypes[ArchetypeIndex]);
			}
			TotalCreatedCount += Counts[ArchetypeIndex];
		}
		AITEST_EQUAL("The total number of entities must match the number created", EntityManager->DebugGetEntityCount(), TotalCreatedCount);
		AITEST_EQUAL("10 entities of FloatsArchetype should have been created", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 10);
		AITEST_EQUAL("100 entities of IntsArchetype should have been created", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 100);
		AITEST_EQUAL("1000 entities of FloatsIntsArchetype should have been created", EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 1000);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_MultipleEntitiesCreation, "System.Mass.Entity.MultipleEntitiesCreation");


struct FEntityTest_EntityBatchCreation : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const int32 Count = 123;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);
		AITEST_EQUAL("Batch creation should create the expected number of entities", Entities.Num(), Count);
		AITEST_EQUAL("The total number of entities present must match the number requested", EntityManager->DebugGetEntityCount(), Count);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_EntityBatchCreation, "System.Mass.Entity.BatchCreation");

struct FEntityTest_BatchCreatingSingleEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, /*Count=*/1, Entities);
		AITEST_EQUAL("Batch creation should have created a single entity", Entities.Num(), 1);
		AITEST_EQUAL("The total number of entities present must match the number created by batch creation", EntityManager->DebugGetEntityCount(), 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_BatchCreatingSingleEntity, "System.Mass.Entity.BatchCreatingSingleEntity");


struct FEntityTest_EntityCreation : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);		
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		AITEST_EQUAL("There should be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("Entity\'s archetype should be the Float one", EntityManager->GetArchetypeForEntity(Entity), FloatsArchetype);
		AITEST_EQUAL("The created entity should have been added to the Floats archetype", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 1);
		AITEST_EQUAL("Other archetypes should not get affected", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) + EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_EntityCreation, "System.Mass.Entity.EntityCreation");


struct FEntityTest_EntityCreationFromInstances : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle Entity = EntityManager->CreateEntity(MakeArrayView(&InstanceInt, 1));
		AITEST_EQUAL("There should be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("Entity\'s archetype should be the Ints one", EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);
		AITEST_EQUAL("The created entity should have been added to the Ints archetype", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 1);
		AITEST_EQUAL("The entity should have the new component with the correct value set", EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity).Value, FTestFragment_Int::TestIntValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_EntityCreationFromInstances, "System.Mass.Entity.EntityCreationFromInstances");

#if 0 // this test is compiled out since AddFragmentToEntity will fails an ensure if a redundant fragment gets added
struct FEntityTest_AddingRedundantFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_Float::StaticStruct());		
		AITEST_EQUAL("Adding a fragment that a given entity\s archetype already has should do nothing", EntityManager->GetArchetypeForEntity(Entity), FloatsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_AddingRedundantFragment, "System.Mass.Entity.AddingRedundantFragment");
#endif //0


struct FEntityTest_AddingFragmentType : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_Int::StaticStruct());
		AITEST_EQUAL("There should be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);
		AITEST_EQUAL("The destination archetype should now store a single entity", EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 1);
		AITEST_EQUAL("The remaining archetype should not be affected", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 0);
		// this test was originally failing due to FEntityData.CurrentArchetype not getting updated during entity moving between archetypes
		AITEST_EQUAL("The entity should get associated with the new archetype", EntityManager->GetArchetypeForEntity(Entity), FloatsIntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_AddingFragmentType, "System.Mass.Entity.AddingFragmentType");

struct FEntityTest_AddingFragmentInstance : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		EntityManager->AddFragmentInstanceListToEntity(Entity, MakeArrayView(&InstanceInt, 1));
		AITEST_EQUAL("There should be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);
		AITEST_EQUAL("The destination archetype should now store a single entity", EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 1);
		AITEST_EQUAL("The archetype containing just the new fragment should not be affected", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 0);
		// this test was originally failing due to FEntityData.CurrentArchetype not getting updated during entity moving between archetypes
		AITEST_EQUAL("The entity should get associated with the new archetype", EntityManager->GetArchetypeForEntity(Entity), FloatsIntsArchetype);
		AITEST_EQUAL("The entity should have the new component with the correct value set", EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity).Value, FTestFragment_Int::TestIntValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_AddingFragmentInstance, "System.Mass.Entity.AddingFragmentType");


struct FEntityTest_RemovingFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsIntsArchetype);
		EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_Float::StaticStruct());
		AITEST_EQUAL("There should be just one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 0);
		AITEST_EQUAL("The destination archetype should now store a single entity", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 1);
		AITEST_EQUAL("The remaining archetype should not be affected", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);
		// this test was originally failing due to FEntityData.CurrentArchetype not getting updated during entity moving between archetypes
		AITEST_EQUAL("The entity should get associated with the new archetype", EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_RemovingFragment, "System.Mass.Entity.RemovingFragment");

struct FEntityTest_RemovingLastFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_Float::StaticStruct());
		AITEST_EQUAL("There should be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);
		// this test was originally failing due to FEntityData.CurrentArchetype not getting updated during entity moving between archetypes
		AITEST_EQUAL("The entity should not get associated to any archetype", EntityManager->GetArchetypeForEntity(Entity), EmptyArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_RemovingLastFragment, "System.Mass.Entity.RemovingLastFragment");

struct FEntityTest_DestroyEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		AITEST_EQUAL("The entity should get associated to the right archetype", EntityManager->GetArchetypeForEntity(Entity), FloatsArchetype);
		EntityManager->DestroyEntity(Entity);
		AITEST_EQUAL("There should not be any entity across the whole system", EntityManager->DebugGetEntityCount(), 0);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_DestroyEntity, "System.Mass.Entity.DestroyEntity");

struct FEntityTest_EntityReservationAndBuilding : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
		AITEST_TRUE("The reserved entity should be a valid entity", EntityManager->IsEntityValid(ReservedEntity));
		AITEST_FALSE("The reserved entity should not be a valid entity", EntityManager->IsEntityBuilt(ReservedEntity));
		EntityManager->BuildEntity(ReservedEntity, FloatsArchetype);
		AITEST_TRUE("The reserved entity should be a valid entity", EntityManager->IsEntityBuilt(ReservedEntity));
		AITEST_EQUAL("There should be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("Entity\'s archetype should be the Float one", EntityManager->GetArchetypeForEntity(ReservedEntity), FloatsArchetype);
		AITEST_EQUAL("The created entity should have been added to the Floats archetype", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 1);
		AITEST_EQUAL("Other archetypes should not get affected", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype) + EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), 0);
		EntityManager->DestroyEntity(ReservedEntity);
		AITEST_EQUAL("There should not be any entity across the whole system", EntityManager->DebugGetEntityCount(), 0);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_EntityReservationAndBuilding, "System.Mass.Entity.EntityReservationAndBuilding");

struct FEntityTest_EntityReservationAndBuildingFromInstances : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
		AITEST_TRUE("The reserved entity should be a valid entity", EntityManager->IsEntityValid(ReservedEntity));
		AITEST_FALSE("The reserved entity should not be a valid entity", EntityManager->IsEntityBuilt(ReservedEntity));
		EntityManager->BuildEntity(ReservedEntity, MakeArrayView(&InstanceInt, 1));
		AITEST_TRUE("The reserved entity should not be a valid entity", EntityManager->IsEntityBuilt(ReservedEntity));
		AITEST_EQUAL("There should be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("Entity\'s archetype should be the Ints one", EntityManager->GetArchetypeForEntity(ReservedEntity), IntsArchetype);
		AITEST_EQUAL("The created entity should have been added to the Ints archetype", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 1);
		AITEST_EQUAL("The entity should have the new component with the correct value set", EntityManager->GetFragmentDataChecked<FTestFragment_Int>(ReservedEntity).Value, FTestFragment_Int::TestIntValue);
		EntityManager->DestroyEntity(ReservedEntity);
		AITEST_EQUAL("There should not be any entity across the whole system", EntityManager->DebugGetEntityCount(), 0);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_EntityReservationAndBuildingFromInstances, "System.Mass.Entity.EntityReservationAndBuildingFromInstances");

struct FEntityTest_ReleaseEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
		AITEST_TRUE("The reserved entity should be a valid entity", EntityManager->IsEntityValid(ReservedEntity));
		AITEST_FALSE("The reserved entity should not be a valid entity", EntityManager->IsEntityBuilt(ReservedEntity));
		AITEST_EQUAL("There should only be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("The entity should not get associated to any archetype", EntityManager->GetArchetypeForEntity(ReservedEntity), FMassArchetypeHandle());
		EntityManager->ReleaseReservedEntity(ReservedEntity);
		AITEST_EQUAL("There should not be any entity across the whole system", EntityManager->DebugGetEntityCount(), 0);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_ReleaseEntity, "System.Mass.Entity.ReleaseEntity");

struct FEntityTest_ReserveAPreviouslyBuiltEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		{
			const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
			AITEST_EQUAL("The entity should get associated to the right archetype", EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);
			EntityManager->DestroyEntity(Entity);
			AITEST_EQUAL("There should not be any entity across the whole system", EntityManager->DebugGetEntityCount(), 0);
			AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(IntsArchetype), 0);
		}
		
		const FMassEntityHandle ReservedEntity = EntityManager->ReserveEntity();
		AITEST_TRUE("The reserved entity should be a valid entity", EntityManager->IsEntityValid(ReservedEntity));
		AITEST_FALSE("The reserved entity should not be a valid entity", EntityManager->IsEntityBuilt(ReservedEntity));
		AITEST_EQUAL("There should only be one entity across the whole system", EntityManager->DebugGetEntityCount(), 1);
		AITEST_EQUAL("The entity should not get associated to any archetype", EntityManager->GetArchetypeForEntity(ReservedEntity), FMassArchetypeHandle());
		EntityManager->BuildEntity(ReservedEntity, FloatsArchetype);
		AITEST_TRUE("The reserved entity should not be a valid entity", EntityManager->IsEntityBuilt(ReservedEntity));
		AITEST_EQUAL("The entity should get associated to the right archetype", EntityManager->GetArchetypeForEntity(ReservedEntity), FloatsArchetype);
		EntityManager->DestroyEntity(ReservedEntity);
		AITEST_EQUAL("There should not be any entity across the whole system", EntityManager->DebugGetEntityCount(), 0);
		AITEST_EQUAL("The original archetype should now have no entities", EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype), 0);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_ReserveAPreviouslyBuiltEntity, "System.Mass.Entity.ReserveAPreviouslyBuiltEntity");

#endif // WITH_MASSENTITY_DEBUG

// testing handling of fragments containing shared ptrs
struct FEntityTest_SharedPtrFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 3, Entities);

		TArray<TWeakPtr<int32>> SharedPtrs;
		for (int32 Index = 0; Index < Entities.Num(); ++Index)
		{
			TSharedPtr<int32> TestData = MakeShared<int32>(Index);
			SharedPtrs.Add(TestData.ToWeakPtr());

			TArray<FInstancedStruct> Array;
			Array.AddZeroed();
			Array[0].InitializeAs<FFragmentWithSharedPtr>(TestData);

			const FMassEntityHandle& EntityHandle = Entities[Index];
			EntityManager->AddFragmentInstanceListToEntity(EntityHandle, Array);
		}

		for (int32 Index = 0; Index < Entities.Num(); ++Index)
		{
			TSharedPtr<int32> TestData = SharedPtrs[Index].Pin();
			const FMassEntityHandle& EntityHandle = Entities[Index];
			const FFragmentWithSharedPtr& Fragment = EntityManager->GetFragmentDataChecked<FFragmentWithSharedPtr>(EntityHandle);
			AITEST_EQUAL("Data stored in fragments is the same as the data provided initially", *Fragment.Data, *TestData);
		}

		EntityManager->AddTagToEntity(Entities[0], FTestTag_A::StaticStruct());
		EntityManager->AddFragmentToEntity(Entities[1], FTestFragment_Float::StaticStruct());

		for (int32 Index = 0; Index < Entities.Num(); ++Index)
		{
			TSharedPtr<int32> TestData = SharedPtrs[Index].Pin();
			const FMassEntityHandle& EntityHandle = Entities[Index];
			const FFragmentWithSharedPtr& Fragment = EntityManager->GetFragmentDataChecked<FFragmentWithSharedPtr>(EntityHandle);
			AITEST_EQUAL("After move operations: Data stored in fragments is the same as the data provided initially", *Fragment.Data, *TestData);
		}

		EntityManager->BatchDestroyEntities(Entities);
		for (int32 Index = 0; Index < Entities.Num(); ++Index)
		{
			TSharedPtr<int32> TestData = SharedPtrs[Index].Pin();
			AITEST_FALSE("After deletion we expect shared data to be released", TestData.IsValid());
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_SharedPtrFragment, "System.Mass.Entity.SharedPtrFragment");

struct FEntityTest_CreationPatterns : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> Entities;

		// create entities straight from fragment instances
		TArray<FInstancedStruct> FragmentInstances = {
			FInstancedStruct::Make<FTestFragment_Int>()
			, FInstancedStruct::Make<FTestFragment_Float>()
		};
		Entities.Add(EntityManager->CreateEntity(FragmentInstances));

		// create the archetype first, and then create an entity within it.
		TArray<const UScriptStruct*> FragmentTypes = {
			FTestFragment_Int::StaticStruct()
			, FTestFragment_Float::StaticStruct()
		};
		FMassArchetypeHandle ArchetypeHandle = EntityManager->CreateArchetype(FragmentTypes);
		Entities.Add(EntityManager->CreateEntity(ArchetypeHandle));

		// Reserve + Build
		// the call will result in ReservedEntity being a valid handle, but not associated with any Archetype data
		Entities.Add(EntityManager->ReserveEntity());

		// any time can pass between reservation and building.
		// the handle can be used to issue commands
		//EntityManager->Defer().AddFragment<FTestFragment_Bool>(Entities.Last());

		// the following call will instantiate the entity
		EntityManager->BuildEntity(Entities.Last(), ArchetypeHandle);
		// if the commands are flushed before the entity is explicitly built then it will be
		// done automatically, but subsequent explicit Build calls will fail

		// entity builder, new in 5.6. There's more, see MassEntityBuilderTest.cpp
		Entities.Add(EntityManager->MakeEntityBuilder().SetForceDeferredCommit(false)
			.Add<FTestFragment_Int>(1024)
			.Add<FTestFragment_Float>(3.14f)
			.Commit());

		for (FMassEntityHandle EntityHandle : Entities)
		{
			const FMassArchetypeHandle ResultArchetypeHandle = EntityManager->GetArchetypeForEntity(EntityHandle);
			AITEST_TRUE("Each method creates an entity in the same archetype", ResultArchetypeHandle == FloatsIntsArchetype);
		}
#if 0
		// This one requires MassGameplay plugin, here for reference.
		UMassSpawnerSubsystem* MassSpawnerSubsystem = UWorld::GetSubsystem<UMassSpawnerSubsystem>(GetWorld());
		FMassEntityTemplateRegistry& TemplateRegistry = MassSpawnerSubsystem->GetMutableTemplateRegistryInstance();

		FMassEntityTemplateData TemplateData;
		TemplateData.AddFragment<FTestFragment_Int>();
		TemplateData.AddFragment<FTestFragment_Float>();
		const TSharedRef<FMassEntityTemplate>& FinalizedTemplate = TemplateRegistry.FindOrAddTemplate(FMassEntityTemplateIDFactory::Make(FGuid::NewGuid()), MoveTemp(TemplateData));

		TArray<FMassEntityHandle> SpawnedEntities;
		MassSpawnerSubsystem->SpawnEntities(FinalizedTemplate, /*NumberToSpawn=*/1, SpawnedEntities);
#endif

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityTest_CreationPatterns, "System.Mass.Entity.CreationPatterns");

} // FMassEntityTestTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
