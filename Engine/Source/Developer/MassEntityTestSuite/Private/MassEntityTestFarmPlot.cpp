// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTestFarmPlot.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityTestFarmPlot)

//@TODO: Can add a ReadyToHarvest tag Fragment on when things are ready to harvest, to stop them ticking and signal that we need to create an icon

//-----------------------------------------------------------------------------
// UFarmProcessorBase
//-----------------------------------------------------------------------------
UFarmProcessorBase::UFarmProcessorBase()
	: EntityQuery(*this)
{
	// not auto-registering to manually control execution
	bAutoRegisterWithProcessingPhases = false;
}

//-----------------------------------------------------------------------------
// UFarmWaterProcessor
//-----------------------------------------------------------------------------
void UFarmWaterProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FFarmWaterFragment>(EMassFragmentAccess::ReadWrite);
}

void UFarmWaterProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UFarmWaterProcessor_Run);
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) {

		const float DeltaTimeSeconds = Context.GetDeltaTimeSeconds();
		TArrayView<FFarmWaterFragment> WaterList = Context.GetMutableFragmentView<FFarmWaterFragment>();

		for (FFarmWaterFragment& WaterFragment : WaterList)
		{
			WaterFragment.CurrentWater = FMath::Clamp(WaterFragment.CurrentWater + WaterFragment.DeltaWaterPerSecond * DeltaTimeSeconds, 0.0f, 1.0f);
		}
	});
}

//-----------------------------------------------------------------------------
// UFarmHarvestTimerSystem_Flowers
//-----------------------------------------------------------------------------
void UFarmHarvestTimerSystem_Flowers::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FHarvestTimerFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FFarmWaterFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FFarmFlowerFragment>(EMassFragmentAccess::ReadWrite);
}

void UFarmHarvestTimerSystem_Flowers::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UFarmHarvestTimerSystem_Flowers_Run);

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) 
		{
			const float WellWateredThreshold = 0.25f;
			TArrayView<FHarvestTimerFragment> TimerList = Context.GetMutableFragmentView<FHarvestTimerFragment>();
			TConstArrayView<FFarmWaterFragment> WaterList = Context.GetFragmentView<FFarmWaterFragment>();
			TArrayView<FFarmFlowerFragment> FlowerList = Context.GetMutableFragmentView<FFarmFlowerFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				if (TimerList[EntityIt].NumSecondsLeft > 0)
				{
					--TimerList[EntityIt].NumSecondsLeft;

					if (WaterList[EntityIt].CurrentWater > WellWateredThreshold)
					{
						++FlowerList[EntityIt].NumBonusTicks;
					}
				}
			}
		});
}

//-----------------------------------------------------------------------------
// UFarmHarvestTimerSystem_Crops
//-----------------------------------------------------------------------------
void UFarmHarvestTimerSystem_Crops::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FHarvestTimerFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FFarmWaterFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FFarmCropFragment>(EMassFragmentAccess::None, EMassFragmentPresence::All);
}

void UFarmHarvestTimerSystem_Crops::Execute(FMassEntityManager & EntityManager, FMassExecutionContext & Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UFarmHarvestTimerSystem_Crops_Run);

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) 
		{
			const float WellWateredThreshold = 0.25f;
			TArrayView<FHarvestTimerFragment> TimerList = Context.GetMutableFragmentView<FHarvestTimerFragment>();
			TConstArrayView<FFarmWaterFragment> WaterList = Context.GetFragmentView<FFarmWaterFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const uint32 TimeToSubtract = (WaterList[EntityIt].CurrentWater > WellWateredThreshold) ? 2 : 1;
				TimerList[EntityIt].NumSecondsLeft = (TimerList[EntityIt].NumSecondsLeft >= TimeToSubtract) ? (TimerList[EntityIt].NumSecondsLeft - TimeToSubtract) : 0;
			}
		});
}

//-----------------------------------------------------------------------------
// UFarmHarvestTimerExpired
//-----------------------------------------------------------------------------
void UFarmHarvestTimerExpired::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FHarvestTimerFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddTagRequirement<FFarmJustBecameReadyToHarvestTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FFarmReadyToHarvestTag>(EMassFragmentPresence::None);
}

void UFarmHarvestTimerExpired::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UFarmHarvestTimerExpired_Run);

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) 
		{
			TConstArrayView<FHarvestTimerFragment> TimerList = Context.GetFragmentView<FHarvestTimerFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				if (TimerList[EntityIt].NumSecondsLeft == 0)
				{
					Context.Defer().AddTag<FFarmJustBecameReadyToHarvestTag>(Context.GetEntity(EntityIt));
				}
			}
		});
}

//-----------------------------------------------------------------------------
// UFarmHarvestTimerSetIcon
//-----------------------------------------------------------------------------
void UFarmHarvestTimerSetIcon::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FFarmGridCellData>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FFarmVisualFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddTagRequirement<FFarmJustBecameReadyToHarvestTag>(EMassFragmentPresence::All);
}

void UFarmHarvestTimerSetIcon::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (HarvestIconISMC == nullptr)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(SET_ICON_SET_ICON_SET_ICON);

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context) 
		{
			TConstArrayView<FFarmGridCellData> GridCoordList = Context.GetFragmentView<FFarmGridCellData>();
			TArrayView<FFarmVisualFragment> VisualList = Context.GetMutableFragmentView<FFarmVisualFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FFarmGridCellData& GridCells = GridCoordList[EntityIt];

				const FVector IconPosition(GridCells.CellX*GridCellWidth, GridCells.CellY*GridCellHeight, HarvestIconHeight);
				const FTransform IconTransform(FQuat::Identity, IconPosition, FVector(HarvestIconScale, HarvestIconScale, HarvestIconScale));

				VisualList[EntityIt].HarvestIconIndex = HarvestIconISMC->AddInstance(IconTransform);

				FMassEntityHandle ThisEntity = Context.GetEntity(EntityIt);
				Context.Defer().RemoveTag<FFarmJustBecameReadyToHarvestTag>(ThisEntity);
				Context.Defer().AddTag<FFarmReadyToHarvestTag>(ThisEntity);
			}
		});
}

//-----------------------------------------------------------------------------
// ALWFragmentTestFarmPlot
//-----------------------------------------------------------------------------
AMassEntityTestFarmPlot::AMassEntityTestFarmPlot()
	: SharedEntityManager(MakeShareable(new FMassEntityManager(this)))
{
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;

	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComp"));

	RootComponent = SceneComponent;

	HarvestIconISMC = CreateDefaultSubobject<UHierarchicalInstancedStaticMeshComponent>(TEXT("HarvestIconISMC"));
	HarvestIconISMC->SetupAttachment(SceneComponent);
	HarvestIconISMC->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
}

void AMassEntityTestFarmPlot::AddItemToGrid(FMassEntityManager& InEntityManager, uint16 X, uint16 Y, FMassArchetypeHandle Archetype, uint16 VisualIndex)
{
	FMassEntityHandle NewItem = InEntityManager.CreateEntity(Archetype);
	PlantedSquares[X + Y * GridWidth] = NewItem;

	InEntityManager.GetFragmentDataChecked<FFarmWaterFragment>(NewItem).DeltaWaterPerSecond = FMath::FRandRange(-0.01f, -0.001f);
	InEntityManager.GetFragmentDataChecked<FHarvestTimerFragment>(NewItem).NumSecondsLeft = 5 + (FMath::Rand() % 100);

	FFarmGridCellData GridCoords;
	GridCoords.CellX = X;
	GridCoords.CellY = Y;
	InEntityManager.GetFragmentDataChecked<FFarmGridCellData>(NewItem) = GridCoords;

	const FVector MeshPosition(X*GridCellWidth, Y*GridCellHeight, 0.0f);
	const FRotator MeshRotation(0.0f, FMath::FRand()*360.0f, 0.0f);
	const FVector MeshScale(1.0f, 1.0f, 1.0f); //@TODO: plumb in scale param?
	const FTransform MeshTransform(MeshRotation, MeshPosition, MeshScale);

	FFarmVisualFragment& VisualComp = InEntityManager.GetFragmentDataChecked<FFarmVisualFragment>(NewItem);
	VisualComp.VisualType = VisualIndex;
	VisualComp.InstanceIndex = VisualDataISMCs[VisualComp.VisualType]->AddInstance(MeshTransform);
}

void AMassEntityTestFarmPlot::BeginPlay()
{
	Super::BeginPlay();

	if (TestDataCropIndicies.Num() == 0
		|| TestDataFlowerIndicies.Num() == 0
		|| VisualDataTable.Num() == 0)
	{
		UE_LOGF(LogMass, Error, "%ls is misconfigured. Make sure TestDataCropIndicies, TestDataFlowerIndicies and VisualDataTable are not empty", *GetName());
		return;
	}

	FMassEntityManager& EntityManager = SharedEntityManager.Get();
#if WITH_MASS_CONCURRENT_RESERVE
	FMassEntityManagerStorageInitParams InitParams;
	InitParams.Emplace<FMassEntityManager_InitParams_Concurrent>();
	EntityManager.Initialize(FMassEntityManagerStorageInitParams(InitParams));
#else
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EntityManager.Initialize();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	FMassArchetypeHandle CropArchetype = EntityManager.CreateArchetype(TArray<const UScriptStruct*>{ FFarmWaterFragment::StaticStruct(), FFarmCropFragment::StaticStruct(), FHarvestTimerFragment::StaticStruct(), FFarmVisualFragment::StaticStruct(), FFarmGridCellData::StaticStruct() });
	FMassArchetypeHandle FlowerArchetype = EntityManager.CreateArchetype(TArray<const UScriptStruct*>{ FFarmWaterFragment::StaticStruct(), FFarmFlowerFragment::StaticStruct(), FHarvestTimerFragment::StaticStruct(), FFarmVisualFragment::StaticStruct(), FFarmGridCellData::StaticStruct() });

	PerFrameSystems.Add(NewObject<UFarmWaterProcessor>(this));
	PerFrameSystems.Last()->CallInitialize(this, EntityManager.AsShared());

	PerSecondSystems.Add(NewObject<UFarmHarvestTimerSystem_Flowers>(this));
	PerSecondSystems.Add(NewObject<UFarmHarvestTimerSystem_Crops>(this));
	PerSecondSystems.Add(NewObject<UFarmHarvestTimerExpired>(this));

	UFarmHarvestTimerSetIcon* IconSetter = NewObject<UFarmHarvestTimerSetIcon>(this);
	IconSetter->HarvestIconISMC = HarvestIconISMC;
	IconSetter->GridCellWidth = GridCellWidth;
	IconSetter->GridCellHeight = GridCellHeight;
	IconSetter->HarvestIconHeight = 200.0f;
	IconSetter->HarvestIconScale = HarvestIconScale;
	PerSecondSystems.Add(IconSetter);

	for (UMassProcessor* Processor : PerSecondSystems)
	{
		Processor->CallInitialize(this, EntityManager.AsShared());
	}

	HarvestIconISMC->SetCullDistances(IconNearCullDistance, IconFarCullDistance);

	// Create ISMCs for each mesh type
	for (const FFarmVisualDataRow& VisualData : VisualDataTable)
	{
		UHierarchicalInstancedStaticMeshComponent* HISMC = NewObject<UHierarchicalInstancedStaticMeshComponent>(this);
		HISMC->SetStaticMesh(VisualData.Mesh);
		if (VisualData.MaterialOverride != nullptr)
		{
			HISMC->SetMaterial(0, VisualData.MaterialOverride);
		}
		HISMC->SetCullDistances(VisualNearCullDistance, VisualFarCullDistance);
		HISMC->SetupAttachment(RootComponent);
		HISMC->RegisterComponent();

		VisualDataISMCs.Add(HISMC);
	}

	// Plant us a grid
	const int32 NumGridCells = GridWidth * GridHeight;
	PlantedSquares.AddDefaulted(NumGridCells);

	for (uint16 Y = 0; Y < GridHeight; ++Y)
	{
		for (uint16 X = 0; X < GridWidth; ++X)
		{
			const bool bSpawnCrop = FMath::FRand() < 0.5f;
			const uint16 VisualIndex = bSpawnCrop ? TestDataCropIndicies[FMath::RandRange(0, TestDataCropIndicies.Num() - 1)] : TestDataFlowerIndicies[FMath::RandRange(0, TestDataFlowerIndicies.Num() - 1)];

			AddItemToGrid(EntityManager, X, Y, bSpawnCrop ? CropArchetype : FlowerArchetype, VisualIndex);
		}
	}
}

void AMassEntityTestFarmPlot::TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction)
{
	Super::TickActor(DeltaTime, TickType, ThisTickFunction);

	if (PlantedSquares.Num() == 0)
	{
		// not configured properly, ignore. 
		return;
	}

	FMassEntityManager& EntityManager = SharedEntityManager.Get();
	
	// Run every frame systems
	{
		FMassProcessingContext Context(EntityManager, DeltaTime);
		UE::Mass::Executor::RunProcessorsView(PerFrameSystems, Context);
	}

	// Run per-second systems when it's time
	NextSecondTimer -= DeltaTime;
	while (NextSecondTimer < 0.0f)
	{
		FMassProcessingContext Context(EntityManager, 1.f);
		UE::Mass::Executor::RunProcessorsView(PerSecondSystems, Context);
		NextSecondTimer += 1.0f;
	}
}

