// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "MassTest"

class FMassEntityTestSuiteModule : public IModuleInterface
{
};

IMPLEMENT_MODULE(FMassEntityTestSuiteModule, MassEntityTestSuite)

#undef LOCTEXT_NAMESPACE
