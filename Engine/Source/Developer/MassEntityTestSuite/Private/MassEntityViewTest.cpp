// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityView.h"


#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test
{
struct FEntityView_Invalidated : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(IntsArchetype);
		FMassEntityView EntityView(*EntityManager.Get(), EntityHandle);

		AITEST_TRUE("The entity view is valid", EntityView.IsValid());

		EntityManager->AddTagToEntity(EntityHandle, FTestTag_A::StaticStruct());

		AITEST_FALSE("(NOT) The entity view is valid", EntityView.IsValid());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_Invalidated, "System.Mass.EntityView.Invalidate");

} // UE::Mass::Test

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
