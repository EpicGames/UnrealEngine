// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassProcessingContext.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//
namespace FMassExecutionTest
{

struct FExecution_Setup : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		AITEST_NOT_NULL("EntitySubsystem needs to be created for the test to be performed", EntityManager.Get());
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecution_Setup, "System.Mass.Execution.Setup");


struct FExecution_EmptyArray : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager); // if EntitySubsystem null InstantTest won't be called at all
		const float DeltaSeconds = 0.f;
		FMassProcessingContext ProcessingContext(*EntityManager, DeltaSeconds);
		// no test performed, let's just see if it results in errors/warnings
		UE::Mass::Executor::RunProcessorsView(TArrayView<UMassProcessor*>(), ProcessingContext);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecution_EmptyArray, "System.Mass.Execution.EmptyArray");


struct FExecution_EmptyPipeline : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager); // if EntitySubsystem null InstantTest won't be called at all
		const float DeltaSeconds = 0.f;
		FMassProcessingContext ProcessingContext(*EntityManager, DeltaSeconds);
		FMassRuntimePipeline Pipeline;
		// no test performed, let's just see if it results in errors/warnings
		UE::Mass::Executor::Run(Pipeline, ProcessingContext);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecution_EmptyPipeline, "System.Mass.Execution.EmptyPipeline");


#if WITH_MASSENTITY_DEBUG
struct FExecution_SingleNullProcessor : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const float DeltaSeconds = 0.f;
		FMassProcessingContext ProcessingContext(EntityManager, DeltaSeconds);
		TArray<UMassProcessor*> Processors;
		Processors.Add(nullptr);

		MASS_SCOPED_ENSURE_TEST(TEXT("Processors contains nullptr"), 1);
		// note that using RunProcessorsView is to bypass reasonable tests UE::Mass::Executor::Run(Pipeline,...) does that are 
		// reported via ensures which are not handled by the automation framework
		UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecution_SingleNullProcessor, "System.Mass.Execution.SingleNullProcessor");


struct FExecution_SingleValidProcessor : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const float DeltaSeconds = 0.f;
		FMassProcessingContext ProcessingContext(EntityManager, DeltaSeconds);
		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		check(Processor);
		// need to set up some requirements to make EntityQuery valid
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		// nothing should break. The actual result of processing is getting tested in MassProcessorTest.cpp
		UE::Mass::Executor::Run(*Processor, ProcessingContext);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecution_SingleValidProcessor, "System.Mass.Execution.SingleValidProcessor");


struct FExecution_MultipleNullProcessors : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const float DeltaSeconds = 0.f;
		FMassProcessingContext ProcessingContext(EntityManager, DeltaSeconds);
		TArray<UMassProcessor*> Processors;
		Processors.Add(nullptr);
		Processors.Add(nullptr);
		Processors.Add(nullptr);

		MASS_SCOPED_ENSURE_TEST(TEXT("Processors contains nullptr"), 1);
		// note that using RunProcessorsView is to bypass reasonable tests UE::Mass::Executor::Run(Pipeline,...) does that are 
		// reported via ensures which are not handled by the automation framework
		UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecution_MultipleNullProcessors, "System.Mass.Execution.MultipleNullProcessors");
#endif // WITH_MASSENTITY_DEBUG


struct FExecution_Sparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const float DeltaSeconds = 0.f;
		FMassProcessingContext ProcessingContext(*EntityManager, DeltaSeconds);
		UMassTestProcessorBase* Processor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		check(Processor);
		// need to set up some requirements to make EntityQuery valid
		Processor->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);

		FMassRuntimePipeline Pipeline;
		{
			TArray<UMassProcessor*> Processors;
			Processors.Add(Processor);
			Pipeline.SetProcessors(Processors);
		}

		FMassArchetypeEntityCollection EntityCollection(FloatsArchetype);
		// nothing should break. The actual result of processing is getting tested in MassProcessorTest.cpp
		
		UE::Mass::Executor::RunWithCollection(Pipeline, ProcessingContext, EntityCollection);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FExecution_Sparse, "System.Mass.Execution.Sparse");
} // FMassExecutionTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
