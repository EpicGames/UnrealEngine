// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassProcessingContext.h"
#include "MassEntityTestTypes.h"
#include "MassEntitySettings.h"
#include "MassExecutor.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//
namespace FMassMultiThreadingTest
{

template<typename T>
static FName GetProcessorName()
{
	return T::StaticClass()->GetFName();
}

struct FMTTestBase : FEntityTestBase
{
	using Super = FExecutionTestBase;
	UMassCompositeProcessor* CompositeProcessor = nullptr;
	TArray<UMassTestProcessorBase*> Processors;
	TArray<FMassProcessorOrderInfo> Result;
	FGraphEventRef FinishEvent;
	TArray<FMassEntityHandle> Entities;

	virtual bool Update() override
	{
		// cannot do this in SetUp without adding a new virtual function for subtests to override
		if (CompositeProcessor == nullptr)
		{
			CompositeProcessor = NewObject<UMassCompositeProcessor>();
			CompositeProcessor->SetGroupName(TEXT("Test"));
			CompositeProcessor->SetProcessors(MakeArrayView<UMassProcessor*>((UMassProcessor**)Processors.GetData(), Processors.Num()));

			FMassProcessingContext Context(*EntityManager);
			FinishEvent = UE::Mass::Executor::TriggerParallelTasks(*CompositeProcessor, MoveTemp(Context), []() {});
		}

		if (FinishEvent->IsComplete())
		{
			// signal that we're done with this test
			return true;
		}
		return false;
	}
};

struct FMTTrivial : FMTTestBase
{
	using Super = FMTTestBase;
	const int32 NumToCreate = 200;
	int32 NumProcessed = 0;

	virtual bool SetUp() override
	{	
		if (!Super::SetUp())
		{
			return false;
		}

		EntityManager->BatchCreateEntities(IntsArchetype, NumToCreate, Entities);

		Processors.Reset();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));

			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			Proc->ForEachEntityChunkExecutionFunction = [this](FMassExecutionContext& Context)
			{
				NumProcessed += Context.GetNumEntities();
			};
		}

		return true;
	}
	
	virtual void VerifyLatentResults() override
	{
		AITEST_EQUAL_LATENT("Expected to process all the created entities.", NumToCreate, NumProcessed);
	}
};
IMPLEMENT_AI_LATENT_TEST(FMTTrivial, "System.Mass.Multithreading.Trivial");


struct FMTBasic : FMTTestBase
{
	using Super = FMTTestBase;
	const int32 NumToCreate = 200;
	int32 NumProcessed = 0;

	virtual bool SetUp() override
	{
		if (!Super::SetUp())
		{
			return false;
		}

		EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumToCreate, Entities);

		Processors.Reset();
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_C>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_B>());
			Proc->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Proc->ForEachEntityChunkExecutionFunction = [this](FMassExecutionContext& Context)
			{
				const TArrayView<FTestFragment_Int> IntsList = Context.GetMutableFragmentView<FTestFragment_Int>();
				const TConstArrayView<FTestFragment_Float> FloatsList = Context.GetFragmentView<FTestFragment_Float>();
				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					IntsList[i].Value = int(FloatsList[i].Value) + IntsList[i].Value;
				}
			};
		} 
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_B>(EntityManager));
			Proc->GetMutableExecutionOrder().ExecuteAfter.Add(GetProcessorName<UMassTestProcessor_A>());
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			Proc->EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
			Proc->ForEachEntityChunkExecutionFunction = [this](FMassExecutionContext& Context)
			{
				const TConstArrayView<FTestFragment_Int> IntsList = Context.GetFragmentView<FTestFragment_Int>();
				const TArrayView<FTestFragment_Float> FloatsList = Context.GetMutableFragmentView<FTestFragment_Float>();
				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					FloatsList[i].Value = float(IntsList[i].Value * IntsList[i].Value);
				}
			};
		} 
		{
			UMassTestProcessorBase* Proc = Processors.Add_GetRef(NewTestProcessor<UMassTestProcessor_A>(EntityManager));
			Proc->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
			Proc->ForEachEntityChunkExecutionFunction = [this](FMassExecutionContext& Context)
			{
				int Index = 0;
				const TArrayView<FTestFragment_Int> IntsList = Context.GetMutableFragmentView<FTestFragment_Int>();
				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					IntsList[i].Value = Index++;
				}
			};
		}

		return true;
	}

	virtual void VerifyLatentResults() override
	{
		for (int i = 0; i < Entities.Num(); ++i)
		{
			FMassEntityView View(FloatsIntsArchetype, Entities[i]);
			AITEST_EQUAL_LATENT(TEXT("Should have predicted values"), View.GetFragmentData<FTestFragment_Int>().Value, i*i + i);
		}
	}
};
IMPLEMENT_AI_LATENT_TEST(FMTBasic, "System.Mass.Multithreading.Basic");

} // FMassMultiThreadingTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
