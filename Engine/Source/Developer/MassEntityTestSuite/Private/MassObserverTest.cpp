// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassProcessingTypes.h"
#include "MassEntityTestTypes.h"
#include "MassEntityTypes.h"
#include "MassEntityUtils.h"
#include "MassCommandBuffer.h"
#include "MassExecutionContext.h"
#include "MassObserverNotificationTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//
namespace UE::Mass::Test::Observers
{

auto EntityIndexSorted = [](const FMassEntityHandle& A, const FMassEntityHandle& B)
{
	return A.Index < B.Index;
};

struct FTagBaseOperation : FEntityTestBase
{
	using FTagStruct = FTestTag_A;

	TArray<FMassEntityHandle> AffectedEntities;
	UMassTestProcessorBase* ObserverProcessor = nullptr;
	EMassObservedOperationFlags OperationFlagsObserved = EMassObservedOperationFlags::None;
	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	TArray<FMassEntityHandle> ExpectedEntities;
	bool bCommandsFlushed = false;

	// @return signifies if the test can continue
	virtual bool PerformOperation() { return false; }

	virtual bool SetUp() override
	{
		if (FEntityTestBase::SetUp())
		{
			ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
			ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			ObserverProcessor->EntityQuery.AddTagRequirement<FTagStruct>(EMassFragmentPresence::All);
			ObserverProcessor->ForEachEntityChunkExecutionFunction = [bCommandsFlushedPtr = &bCommandsFlushed, AffectedEntitiesPtr = &AffectedEntities](FMassExecutionContext& Context)
			{
				AffectedEntitiesPtr->Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
				Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushedPtr](FMassEntityManager&)
					{
						// dummy command, here just to catch if commands issue by observers got executed at all
						*bCommandsFlushedPtr = true;
					});
			};

			return true;
		}
		return false;
	}

	virtual bool InstantTest() override
	{
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

		if (PerformOperation())
		{
			EntityManager->FlushCommands();
			AITEST_EQUAL(TEXT("The observer is expected to be run for predicted number of entities"), AffectedEntities.Num(), ExpectedEntities.Num());
			AITEST_TRUE(TEXT("The commands issued by the observer are flushed"), bCommandsFlushed);

			ExpectedEntities.Sort(EntityIndexSorted);
			AffectedEntities.Sort(EntityIndexSorted);

			for (int i = 0; i < ExpectedEntities.Num(); ++i)
			{
				AITEST_EQUAL(TEXT("Expected and affected sets should be the same"), AffectedEntities[i], ExpectedEntities[i]);
			}
		}

		return true;
	}
};

struct FSingleEntitySingleArchetypeAdd : FTagBaseOperation
{
	FSingleEntitySingleArchetypeAdd() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }
	virtual bool PerformOperation() override 
	{
		ExpectedEntities = { EntitiesInt[1] };
		EntityManager->Defer().AddTag<FTagStruct>(EntitiesInt[1]);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSingleEntitySingleArchetypeAdd, "System.Mass.Observer.Tag.SingleEntitySingleArchetypeAdd");

struct FSingleEntitySingleArchetypeRemove : FTagBaseOperation
{
	FSingleEntitySingleArchetypeRemove() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }
	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[1] };

		EntityManager->Defer().AddTag<FTagStruct>(EntitiesInt[1]);
		EntityManager->FlushCommands();
		// since we're only observing tag removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Tag addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		EntityManager->Defer().RemoveTag<FTagStruct>(EntitiesInt[1]);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSingleEntitySingleArchetypeRemove, "System.Mass.Observer.Tag.SingleEntitySingleArchetypeRemove");

struct FSingleEntitySingleArchetypeDestroy : FTagBaseOperation
{
	FSingleEntitySingleArchetypeDestroy() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }
	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[1] };
		EntityManager->Defer().AddTag<FTagStruct>(EntitiesInt[1]);
		EntityManager->FlushCommands();
		// since we're only observing tag removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("FTagStruct addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		EntityManager->Defer().DestroyEntity(EntitiesInt[1]);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSingleEntitySingleArchetypeDestroy, "System.Mass.Observer.Tag.SingleEntitySingleArchetypeDestroy");

struct FMultipleArchetypeAdd : FTagBaseOperation
{
	FMultipleArchetypeAdd() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMultipleArchetypeAdd, "System.Mass.Observer.Tag.MultipleArchetypesAdd");


struct FMultipleArchetypeAdd_Sync : FTagBaseOperation
{
	FMultipleArchetypeAdd_Sync() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->AddTagToEntity(ModifiedEntity, FTagStruct::StaticStruct());
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMultipleArchetypeAdd_Sync, "System.Mass.Observer.Tag.MultipleArchetypesAdd_Sync");

struct FMultipleArchetypeRemove : FTagBaseOperation
{
	FMultipleArchetypeRemove() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
		}
		EntityManager->FlushCommands();
		// since we're only observing tag removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("FTagStruct addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().RemoveTag<FTagStruct>(ModifiedEntity);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMultipleArchetypeRemove, "System.Mass.Observer.Tag.MultipleArchetypesRemove");

struct FMultipleArchetypeRemove_Sync : FTagBaseOperation
{
	FMultipleArchetypeRemove_Sync() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->AddTagToEntity(ModifiedEntity, FTagStruct::StaticStruct());
		}
		
		// since we're only observing tag removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("FTagStruct addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->RemoveTagFromEntity(ModifiedEntity, FTagStruct::StaticStruct());
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMultipleArchetypeRemove_Sync, "System.Mass.Observer.Tag.MultipleArchetypesRemove_Sync");

struct FMultipleArchetypeDestroy : FTagBaseOperation
{
	FMultipleArchetypeDestroy() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
		}
		EntityManager->FlushCommands();
		// since we're only observing tag removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Tag addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().DestroyEntity(ModifiedEntity);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMultipleArchetypeDestroy, "System.Mass.Observer.Tag.MultipleArchetypesDestroy");

struct FForbidModifyOnDestroy : FTagBaseOperation
{
	using Super = FTagBaseOperation;
	FForbidModifyOnDestroy() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }

	virtual bool SetUp() override
	{
		if (Super::SetUp())
		{
			ObserverProcessor->ForEachEntityChunkExecutionFunction = [bCommandsFlushedPtr = &bCommandsFlushed, AffectedEntitiesPtr = &AffectedEntities](FMassExecutionContext& Context)
				{
					AffectedEntitiesPtr->Append(Context.GetEntities().GetData(), Context.GetEntities().Num());

					// try changing the input entities' composition.
					const bool bIsProcessing = Context.GetEntityManagerChecked().IsProcessing();
					//for (int32 EntityIndex = 0; EntityIndex < Context.GetEntities().Num(); ++EntityIndex)
					for (const FMassEntityHandle EntityHandle : Context.GetEntities())
					{
						//Context.Defer().AddTag<FTestTag_A>(EntityHandle);
						//Context.GetEntityManagerChecked().AddTagToEntity(EntityHandle, FTestTag_B::StaticStruct());
					}

					Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushedPtr](FMassEntityManager&)
						{
							// dummy command, here just to catch if commands issue by observers got executed at all
							*bCommandsFlushedPtr = true;
						});
				};

			return true;
		}
		return false;
	}

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
		}
		EntityManager->FlushCommands();

		// since we're only observing tag removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Tag addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().DestroyEntity(ModifiedEntity);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FForbidModifyOnDestroy, "System.Mass.Observer.ForbidModifyOnDestroy");

struct FMultipleArchetypeSwap : FTagBaseOperation
{
	FMultipleArchetypeSwap() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesIntsFloat[1], EntitiesInt[0], EntitiesInt[2] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddTag<FTagStruct>(ModifiedEntity);
		}
		EntityManager->FlushCommands();
		// since we're only observing tag removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Tag addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().SwapTags<FTagStruct, FTestTag_B>(ModifiedEntity);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMultipleArchetypeSwap, "System.Mass.Observer.Tag.MultipleArchetypesSwap");

struct FEntityCreation_Individuals : FTagBaseOperation
{
	FEntityCreation_Individuals() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesToSpawnCount = 6;
		
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		int32 ArrayMidPoint = 0;
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnCount, EntitiesInt);
			ArrayMidPoint = EntitiesInt.Num() / 2;

			for (int32 Index = 0; Index < ArrayMidPoint; ++Index)
			{
				EntityManager->AddTagToEntity(EntitiesInt[Index], FTagStruct::StaticStruct());
			}
			AITEST_EQUAL(TEXT("The tag observer is not expected to run yet"), AffectedEntities.Num(), 0);
		}
		AITEST_EQUAL(TEXT("The tag observer is expected to run just after FEntityCreationContext's destruction"), AffectedEntities.Num(), ArrayMidPoint);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityCreation_Individuals, "System.Mass.Observer.Create.TagInvididualEntities");

struct FEntityCreation_Batched : FTagBaseOperation
{
	FEntityCreation_Batched() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesToSpawnCount = 6;

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnCount, EntitiesInt);

			EntityManager->BatchChangeTagsForEntities(CreationContext->GetEntityCollections(*EntityManager.Get()), FMassTagBitSet(FTagStruct::StaticStruct()), FMassTagBitSet());
			AITEST_TRUE(TEXT("The tag observer is not expected to run yet"), AffectedEntities.Num() == 0);
			AITEST_FALSE(TEXT("CreationContext's entity collection should be invalidated at this moment"), CreationContext->DebugAreEntityCollectionsUpToDate());

			EntityManager->BatchChangeTagsForEntities(CreationContext->GetEntityCollections(*EntityManager.Get()), FMassTagBitSet(FTagStruct::StaticStruct()), FMassTagBitSet());
			AITEST_TRUE(TEXT("The tag observer is still not expected to run"), AffectedEntities.Num() == 0);
		}
		AITEST_TRUE(TEXT("The tag observer is expected to run just after FEntityCreationContext's destruction"), AffectedEntities.Num() > 0);
		AITEST_EQUAL(TEXT("The tag observer is expected to process every entity just once"), AffectedEntities.Num(), EntitiesInt.Num());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityCreation_Batched, "System.Mass.Observer.Create.TagBatchedEntities");

//-----------------------------------------------------------------------------
// fragments
//-----------------------------------------------------------------------------
struct FFragmentTestBase : FEntityTestBase
{
	using FFragmentStruct = FTestFragment_Float;

	TArray<FMassEntityHandle> AffectedEntities;
	UMassTestProcessorBase* ObserverProcessor = nullptr;
	//EMassObservedOperation OperationObserved = EMassObservedOperation::MAX;
	EMassObservedOperationFlags OperationFlagsObserved = EMassObservedOperationFlags::None;
	TArray<FMassEntityHandle> EntitiesFloats;
	TArray<FMassEntityHandle> EntitiesInt;
	TArray<FMassEntityHandle> EntitiesIntsFloat;
	TArray<FMassEntityHandle> ExpectedEntities;
	bool bCommandsFlushed = false;

	// @return signifies if the test can continue
	virtual bool PerformOperation() { return false; }

	virtual bool SetUp() override
	{
		if (FEntityTestBase::SetUp())
		{
			ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
			ObserverProcessor->EntityQuery.AddRequirement(FFragmentStruct::StaticStruct(), EMassFragmentAccess::ReadWrite);
			ObserverProcessor->ForEachEntityChunkExecutionFunction = [bCommandsFlushedPtr = &bCommandsFlushed, AffectedEntitiesPtr = &AffectedEntities](FMassExecutionContext& Context)
				{
					AffectedEntitiesPtr->Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
					Context.Defer().PushCommand<FMassDeferredSetCommand>([&bCommandsFlushedPtr](FMassEntityManager&)
						{
							// dummy command, here just to catch if commands issue by observers got executed at all
							*bCommandsFlushedPtr = true;
						});
				};

			return true;
		}
		return false;
	}

	virtual bool InstantTest() override
	{
		EntityManager->BatchCreateEntities(IntsArchetype, 3, EntitiesInt);
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, 3, EntitiesIntsFloat);

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);
				
		if (PerformOperation())
		{
			EntityManager->FlushCommands();
			AITEST_EQUAL(TEXT("The fragment observer is expected to be run for predicted number of entities"), AffectedEntities.Num(), ExpectedEntities.Num());
			AITEST_TRUE(TEXT("The commands issued by the observer are flushed"), bCommandsFlushed);

			ExpectedEntities.Sort(EntityIndexSorted);
			AffectedEntities.Sort(EntityIndexSorted);

			for (int i = 0; i < ExpectedEntities.Num(); ++i)
			{
				AITEST_EQUAL(TEXT("Expected and affected sets should be the same"), AffectedEntities[i], ExpectedEntities[i]);
			}
		}

		return true;
	}
};

struct FFragmentTest_SingleEntitySingleArchetypeAdd : FFragmentTestBase
{
	FFragmentTest_SingleEntitySingleArchetypeAdd() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }
	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[1] };
		EntityManager->Defer().AddFragment<FFragmentStruct>(EntitiesInt[1]);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_SingleEntitySingleArchetypeAdd, "System.Mass.Observer.Fragment.SingleEntitySingleArchetypeAdd");

struct FFragmentTest_SingleEntitySingleArchetypeRemove : FFragmentTestBase
{
	FFragmentTest_SingleEntitySingleArchetypeRemove() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }
	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[1] };

		EntityManager->Defer().AddFragment<FFragmentStruct>(EntitiesInt[1]);
		EntityManager->FlushCommands();
		// since we're only observing Fragment removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Fragment addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		EntityManager->Defer().RemoveFragment<FFragmentStruct>(EntitiesInt[1]);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_SingleEntitySingleArchetypeRemove, "System.Mass.Observer.Fragment.SingleEntitySingleArchetypeRemove");

struct FFragmentTest_SingleEntitySingleArchetypeDestroy : FFragmentTestBase
{
	FFragmentTest_SingleEntitySingleArchetypeDestroy() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }
	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[1] };
		EntityManager->Defer().AddFragment<FFragmentStruct>(EntitiesInt[1]);
		EntityManager->FlushCommands();
		// since we're only observing Fragment removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Fragment addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		EntityManager->Defer().DestroyEntity(EntitiesInt[1]);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_SingleEntitySingleArchetypeDestroy, "System.Mass.Observer.Fragment.SingleEntitySingleArchetypeDestroy");

struct FFragmentTest_MultipleArchetypeAdd : FFragmentTestBase
{
	FFragmentTest_MultipleArchetypeAdd() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesInt[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddFragment<FFragmentStruct>(ModifiedEntity);
		}
		// also adding the fragment to the other archetype that already has the fragment. This should not yield any results
		for (const FMassEntityHandle& OtherEntity : EntitiesIntsFloat)
		{
			EntityManager->Defer().AddFragment<FFragmentStruct>(OtherEntity);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_MultipleArchetypeAdd, "System.Mass.Observer.Fragment.MultipleArchetypesAdd");

struct FFragmentTest_MultipleArchetypeRemove : FFragmentTestBase
{
	FFragmentTest_MultipleArchetypeRemove() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddFragment<FFragmentStruct>(ModifiedEntity);
		}
		EntityManager->FlushCommands();
		// since we're only observing Fragment removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Fragment addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().RemoveFragment<FFragmentStruct>(ModifiedEntity);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_MultipleArchetypeRemove, "System.Mass.Observer.Fragment.MultipleArchetypesRemove");

struct FFragmentTest_MultipleArchetypeDestroy : FFragmentTestBase
{
	FFragmentTest_MultipleArchetypeDestroy() { OperationFlagsObserved = EMassObservedOperationFlags::Remove; }

	virtual bool PerformOperation() override
	{
		ExpectedEntities = { EntitiesInt[0], EntitiesInt[2], EntitiesIntsFloat[1] };
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().AddFragment<FFragmentStruct>(ModifiedEntity);
		}
		EntityManager->FlushCommands();
		// since we're only observing Fragment removal we don't expect AffectedEntities to contain any data at this point
		AITEST_EQUAL(TEXT("Fragment addition is not being observed and is not expected to produce results yet"), AffectedEntities.Num(), 0);
		for (const FMassEntityHandle& ModifiedEntity : ExpectedEntities)
		{
			EntityManager->Defer().DestroyEntity(ModifiedEntity);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_MultipleArchetypeDestroy, "System.Mass.Observer.Fragment.MultipleArchetypesDestroy");

struct FFragmentTest_EntityCreation_Individual : FFragmentTestBase
{
	FFragmentTest_EntityCreation_Individual() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool InstantTest() override
	{
		constexpr float TestValue = 123.456f;
		float ValueOnNotification = 0.f;

		ObserverProcessor->ForEachEntityChunkExecutionFunction = [&ValueOnNotification](FMassExecutionContext& Context) //-V1047 - This lambda is cleared before routine exit
			{
				const TConstArrayView<FFragmentStruct> Fragments = Context.GetFragmentView<FFragmentStruct>();
				for (int32 EntityIndex = 0; EntityIndex < Context.GetNumEntities(); EntityIndex++)
				{
					ValueOnNotification = Fragments[EntityIndex].Value;
				};
			};

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		TArray<FInstancedStruct> FragmentInstanceList = { FInstancedStruct::Make(FFragmentStruct(TestValue)) };

		// BuildEntity
		{
			const FMassEntityHandle Entity= EntityManager->ReserveEntity();
			EntityManager->BuildEntity(Entity, FragmentInstanceList);	
			AITEST_EQUAL(TEXT("The fragment observer notified by BuildEntity is expected to be able to fetch the initial value"), ValueOnNotification, TestValue);
			EntityManager->DestroyEntity(Entity);
		}

		// CreateEntity
		{
			ValueOnNotification = 0.f;
			const FMassEntityHandle Entity = EntityManager->CreateEntity(FragmentInstanceList);
			AITEST_EQUAL(TEXT("The fragment observer notified by CreateEntity is expected to be able to fetch the initial value"), ValueOnNotification, TestValue);
			EntityManager->DestroyEntity(Entity);
		}

		ObserverProcessor->ForEachEntityChunkExecutionFunction = nullptr;

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_EntityCreation_Individual, "System.Mass.Observer.Create.FragmentSingleEntity");

struct FFragmentTest_EntityCreation_Individuals : FFragmentTestBase
{
	FFragmentTest_EntityCreation_Individuals() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesToSpawnCount = 6;

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		int32 ArrayMidPoint = 0;
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnCount, EntitiesInt);
			ArrayMidPoint = EntitiesInt.Num() / 2;

			for (int32 Index = 0; Index < ArrayMidPoint; ++Index)
			{
				EntityManager->AddFragmentToEntity(EntitiesInt[Index], FFragmentStruct::StaticStruct());
			}
			AITEST_EQUAL(TEXT("The fragment observer is not expected to run yet"), AffectedEntities.Num(), 0);
		}
		AITEST_EQUAL(TEXT("The fragment observer is expected to run just after FEntityCreationContext's destruction"), AffectedEntities.Num(), ArrayMidPoint);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FFragmentTest_EntityCreation_Individuals, "System.Mass.Observer.Create.FragmentIndividualEntities");

#if WITH_MASSENTITY_DEBUG
struct FObserverChangingComposition_Sync : FFragmentTestBase
{
	FObserverChangingComposition_Sync()
	{
		OperationFlagsObserved = EMassObservedOperationFlags::Add;
	}

	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesToSpawn = 3;
		const FMassArchetypeHandle OriginalArchetype = FloatsArchetype;

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		{
			MASS_SCOPED_ENSURE_TEST("Use asynchronous API instead", 1);

			ObserverProcessor->ForEachEntityChunkExecutionFunction = [EntityManager = EntityManager](FMassExecutionContext& Context)
			{
				EntityManager->AddFragmentToEntity(Context.GetEntity(0), FTestFragment_Int::StaticStruct());
			};

			EntityManager->BatchCreateEntities(OriginalArchetype, EntitiesToSpawn, EntitiesInt);

			AITEST_EQUAL("Number of entities in the original archetype, no moves expected", EntityManager->DebugGetArchetypeEntitiesCount(OriginalArchetype), EntitiesToSpawn);
		}
		{
			MASS_SCOPED_ENSURE_TEST("Use asynchronous API instead", 1);

			ObserverProcessor->ForEachEntityChunkExecutionFunction = [EntityManager = EntityManager, OriginalArchetype](FMassExecutionContext& Context)
			{
				FMassArchetypeEntityCollection EntityCollection(OriginalArchetype, Context.GetEntities(), FMassArchetypeEntityCollection::NoDuplicates);
				EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&EntityCollection, 1)
					, FMassFragmentBitSet(FTestFragment_Int::StaticStruct()), {});
			};

			EntityManager->BatchCreateEntities(OriginalArchetype, EntitiesToSpawn, EntitiesInt);

			AITEST_EQUAL("Number of entities in the original archetype, no moves expected", EntityManager->DebugGetArchetypeEntitiesCount(OriginalArchetype), EntitiesToSpawn * 2);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FObserverChangingComposition_Sync, "System.Mass.Observer.ChangingCompositionSync");

struct FObserverChangingComposition_Deferred : FFragmentTestBase
{
	FObserverChangingComposition_Deferred()
	{
		OperationFlagsObserved = EMassObservedOperationFlags::Add;
	}

	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesToSpawn = 3;
		const FMassArchetypeHandle OriginalArchetype = FloatsArchetype;

		ObserverProcessor->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
			{
				Context.Defer().PushCommand<FMassCommandAddFragments<FTestFragment_Int>>(Context.GetEntities());
			};

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		EntityManager->BatchCreateEntities(OriginalArchetype, EntitiesToSpawn, EntitiesInt);

		AITEST_EQUAL("Number of entities in the original archetype", EntityManager->DebugGetArchetypeEntitiesCount(OriginalArchetype), 0);
		AITEST_EQUAL("Number of entities in the target archetype", EntityManager->DebugGetArchetypeEntitiesCount(FloatsIntsArchetype), EntitiesToSpawn);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FObserverChangingComposition_Deferred, "System.Mass.Observer.ChangingCompositionDeferred");
#endif // WITH_MASSENTITY_DEBUG

/**
 * This test aims to verify expected behavior of observers when there's a creation context active, when composition-mutating
 * operations are affecting entities other than the ones being created.
 */
struct FModificationsWhileCreationContextActive : FTagBaseOperation
{
	FModificationsWhileCreationContextActive() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesToSpawnInFirstBatch = 3;
		constexpr int32 EntitiesToSpawnInSecondBatch = 5;

		EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnInFirstBatch, EntitiesInt);
		FMassArchetypeEntityCollection InitialEntitiesCollection(IntsArchetype, EntitiesInt, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTagStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		{
			TSharedRef<FMassObserverManager::FObserverLock> ObserversLock = EntityManager->GetOrMakeObserversLock();
			{
				TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToSpawnInSecondBatch, EntitiesInt);
				ensure(EntitiesInt.Num() == EntitiesToSpawnInFirstBatch + EntitiesToSpawnInSecondBatch);
				// note that the observers' behavior regarding the entities just created gets tested by FEntityCreation_Batched test above
				// we're testing only the behavior related to the previously created entities here
			}
			EntityManager->BatchChangeTagsForEntities(MakeArrayView(&InitialEntitiesCollection, 1)
				, FMassTagBitSet(FTagStruct::StaticStruct()), FMassTagBitSet());

			AITEST_TRUE(TEXT("The tag observer is not expected to run yet"), AffectedEntities.Num() == 0);
		}
		AITEST_TRUE(TEXT("The tag observer is expected to run just after FEntityCreationContext's destruction"), AffectedEntities.Num() > 0);
		AITEST_EQUAL(TEXT("The tag observer is expected to process only the original entities, that had a tag added to them"), AffectedEntities.Num(), EntitiesToSpawnInFirstBatch);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FModificationsWhileCreationContextActive, "System.Mass.Observer.Create.ModificationsToOtherEntities");

struct FCreationOperationOrder : FFragmentTestBase
{
	int32 Counter = 0;

	FCreationOperationOrder()
	{
		OperationFlagsObserved = EMassObservedOperationFlags::Add;
	}

	virtual void BuildScenario(TArray<FMassEntityHandle>& PreExistingEntities, TArray<FMassEntityHandle>& NewEntities) = 0;

	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> PreExistingEntities;
		TArray<FMassEntityHandle> NewEntities;

		ObserverProcessor->ForEachEntityChunkExecutionFunction = [Counter = &this->Counter](FMassExecutionContext& Context)
			{
				for (auto EntityId : Context.CreateEntityIterator())
				{
					Context.GetMutableFragmentView<FFragmentStruct>()[EntityId].Value = static_cast<float>(++(*Counter));
				}
			};

		EntityManager->BatchCreateEntities(IntsArchetype, 2, PreExistingEntities);

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);

		BuildScenario(PreExistingEntities, NewEntities);

		// the specific order of entities handled within a single creation context doesn't
		// need to match the assumed order
		float FirstBatchValues[] = {
			EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[0]).Value
			, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[1]).Value
		};
		AITEST_TRUE("First batch's values match", (FirstBatchValues[0] == 1.f && FirstBatchValues[1] == 2.f) || (FirstBatchValues[1] == 1.f && FirstBatchValues[0] == 2.f));

		float PreExistingEntitiesValues[] = {
			EntityManager->GetFragmentDataChecked<FTestFragment_Float>(PreExistingEntities[0]).Value
			, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(PreExistingEntities[1]).Value
		};
		AITEST_EQUAL("First preexisting entity's value", PreExistingEntitiesValues[0], 3.f);
		AITEST_EQUAL("Second preexisting entity's value", PreExistingEntitiesValues[1], 6.f);

		float SecondBatchBatchValues[] = {
			EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[2]).Value
			, EntityManager->GetFragmentDataChecked<FTestFragment_Float>(NewEntities[3]).Value
		};
		AITEST_TRUE("First batch's values match", (SecondBatchBatchValues[0] == 4.f && SecondBatchBatchValues[1] == 5.f) || (SecondBatchBatchValues[1] == 4.f && SecondBatchBatchValues[0] == 5.f));

		return true;
	}
};

struct FCreationOperationOrder_Batch : FCreationOperationOrder
{
	virtual void BuildScenario(TArray<FMassEntityHandle>& PreExistingEntities, TArray<FMassEntityHandle>& NewEntities) override
	{
		TSharedRef<FMassObserverManager::FObserverLock> ObserversLock = EntityManager->GetOrMakeObserversLock();
		{
			// creating two separate entities, that should end up in the same creation context
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
			EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
		}
		{
			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&PreExistingEntities[0], 1), FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&Collection, 1), FMassFragmentBitSet(FTestFragment_Float::StaticStruct()), {});
		}
		{
			// creating two separate entities, that should end up in the same creation context
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
			EntityManager->BatchCreateEntities(FloatsArchetype, 1, NewEntities);
		}
		{
			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&PreExistingEntities[1], 1), FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchChangeFragmentCompositionForEntities(MakeArrayView(&Collection, 1), FMassFragmentBitSet(FTestFragment_Float::StaticStruct()), {});
		}
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreationOperationOrder_Batch, "System.Mass.Observer.Create.CreationOperationOrder.Batch");

struct FCreationOperationOrder_Individual : FCreationOperationOrder
{
	virtual void BuildScenario(TArray<FMassEntityHandle>& PreExistingEntities, TArray<FMassEntityHandle>& NewEntities) override
	{
		TSharedRef<FMassObserverManager::FObserverLock> ObserversLock = EntityManager->GetOrMakeObserversLock();
		{
			// creating two separate entities, that should end up in the same creation context
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->GetOrMakeCreationContext();
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
		}
		EntityManager->AddFragmentToEntity(PreExistingEntities[0], FTestFragment_Float::StaticStruct());
		{
			// creating two separate entities, that should end up in the same creation context
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->GetOrMakeCreationContext();
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
			NewEntities.Add(EntityManager->CreateEntity(FloatsArchetype));
		}
		EntityManager->AddFragmentToEntity(PreExistingEntities[1], FTestFragment_Float::StaticStruct());
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreationOperationOrder_Individual, "System.Mass.Observer.Create.CreationOperationOrder.Individual");

// The scenario being tested:
// 1. Create entities with Float fragment
// 2. Add an unobserved tag A - results in created entities changing archetype
// 3. Release the creation context - we expect Float observers to trigger
struct FCreatedEntitiesUnobservedCompositionChange : FFragmentTestBase
{
	FCreatedEntitiesUnobservedCompositionChange() { OperationFlagsObserved = EMassObservedOperationFlags::Add; }

	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesToSpawnCount = 6;

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FFragmentStruct::StaticStruct(), OperationFlagsObserved, ObserverProcessor);
		{
			TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToSpawnCount, EntitiesFloats);

			// add unobserved tag
			EntityManager->BatchChangeTagsForEntities(CreationContext->GetEntityCollections(*EntityManager.Get()), FMassTagBitSet(FTestTag_A::StaticStruct()), FMassTagBitSet());

			AITEST_EQUAL(TEXT("The fragment observer is not expected to run yet"), AffectedEntities.Num(), 0);
		}
		AITEST_EQUAL(TEXT("The fragment observer is expected to run just after FEntityCreationContext's destruction"), AffectedEntities.Num(), EntitiesToSpawnCount);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCreatedEntitiesUnobservedCompositionChange, "System.Mass.Observer.Create.UnobservedCompositionChange");


struct FMoveToAnotherArchetype_SingleEntity : FEntityTestBase
{
	bool bTagAdded = false;
	bool bTagRemoved = false;
	bool bFloatAdded = false;
	bool bFloatRemoved = false;

	virtual bool SetUp() override
	{
		if (FEntityTestBase::SetUp() == false)
		{
			return false;
		}

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();

		auto CreateObserver = [this](const TFunction<void(FMassExecutionContext& Context)>& StoreResultFunction)
		{
			UMassTestProcessorBase* ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
			ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			ObserverProcessor->ForEachEntityChunkExecutionFunction = StoreResultFunction;
			return ObserverProcessor;
		};

		ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Add
			, CreateObserver([this](FMassExecutionContext& Context)
				{
					bTagAdded = true;
				}
			));

		ObserverManager.AddObserverInstance(FTestTag_A::StaticStruct(), EMassObservedOperationFlags::Remove
			, CreateObserver([this](FMassExecutionContext& Context)
				{
					bTagRemoved = true;
				}
			));

		ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add
			, CreateObserver([this](FMassExecutionContext& Context)
				{
					bFloatAdded = true;
				}
			));

		ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Remove
			, CreateObserver([this](FMassExecutionContext& Context)
				{
					bFloatRemoved = true;
				}
			));

		return true;
	}

	virtual bool InstantTest() override
	{
		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(IntsArchetype);

		// create target archetype
		const FMassArchetypeHandle TargetArchetypeHandle = EntityManager->CreateArchetype(IntsArchetype, { FTestTag_A::StaticStruct(), FTestFragment_Float::StaticStruct() });

		EntityManager->MoveEntityToAnotherArchetype(EntityHandle, TargetArchetypeHandle);
		AITEST_TRUE("Tag addition observer has been executed", bTagAdded);
		AITEST_TRUE("Fragment addition observer has been executed", bFloatAdded);
		AITEST_FALSE("(NOT) Tag removal observer has been executed", bTagRemoved);
		AITEST_FALSE("(NOT) Fragment addition observer has been executed", bFloatRemoved);

		// moving back to the original archetype will remove the two added elements, and should trigger observers
		EntityManager->MoveEntityToAnotherArchetype(EntityHandle, IntsArchetype);
		AITEST_TRUE("Tag removal observer has been executed", bTagRemoved);
		AITEST_TRUE("Fragment addition observer has been executed", bFloatRemoved);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FMoveToAnotherArchetype_SingleEntity, "System.Mass.Observer.MoveToAnotherArchetype");
//
//struct FRecursiveObserver : FEntityTestBase
//{
//	bool bTagAdded = false;
//	bool bTagRemoved = false;
//	
//	virtual bool SetUp() override
//	{
//		if (FEntityTestBase::SetUp() == false)
//		{
//			return false;
//		}
//
//		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
//
//		auto CreateObserver = [this](const TFunction<void(FMassExecutionContext& Context)>& StoreResultFunction)
//		{
//			UMassTestProcessorBase* ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
//			ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
//			ObserverProcessor->ForEachEntityChunkExecutionFunction = StoreResultFunction;
//			return ObserverProcessor;
//		};
//
//		ObserverManager.AddObserverInstance(*FTestTag_A::StaticStruct(), EMassObservedOperation::Add
//			, *CreateObserver([this](FMassExecutionContext& Context)
//				{
//					bTagAdded = true;
//				}
//			));
//
//		ObserverManager.AddObserverInstance(*FTestTag_A::StaticStruct(), EMassObservedOperation::Remove
//			, *CreateObserver([this](FMassExecutionContext& Context)
//				{
//					bTagRemoved = true;
//				}
//			));
//
//		return true;
//	}
//
//	virtual bool InstantTest() override
//	{
//		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(IntsArchetype);
//
//		// create target archetype
//		const FMassArchetypeHandle TargetArchetypeHandle = EntityManager->CreateArchetype(IntsArchetype, { FTestTag_A::StaticStruct(), FTestFragment_Float::StaticStruct() });
//
//		EntityManager->MoveEntityToAnotherArchetype(EntityHandle, TargetArchetypeHandle);
//		AITEST_TRUE("Tag addition observer has been executed", bTagAdded);
//		AITEST_TRUE("Fragment addition observer has been executed", bFloatAdded);
//		AITEST_FALSE("(NOT) Tag removal observer has been executed", bTagRemoved);
//		AITEST_FALSE("(NOT) Fragment addition observer has been executed", bFloatRemoved);
//
//		// moving back to the original archetype will remove the two added elements, and should trigger observers
//		EntityManager->MoveEntityToAnotherArchetype(EntityHandle, IntsArchetype);
//		AITEST_TRUE("Tag removal observer has been executed", bTagRemoved);
//		AITEST_TRUE("Fragment addition observer has been executed", bFloatRemoved);
//
//		return true;
//	}
//};
//IMPLEMENT_AI_INSTANT_TEST(FMoveToAnotherArchetype_SingleEntity, "System.Mass.Observer.Recursive");

//----------------------------------------------------------------------//
// Multi-type observer processor tests
//----------------------------------------------------------------------//

/**
 * Registers the same UMassTestProcessorBase instance for two fragment types (Float Add and Int Add).
 * Moving an entity from EmptyArchetype to FloatsIntsArchetype adds both fragments in a single composition
 * change, which produces one HandleElementsImpl call with both types in ObservedTypesOverlap.
 * The deduplication set in HandleElementsImpl must prevent Execute from firing twice.
 */
struct FCompositeObserver_DeduplicationTwoTypes : FEntityTestBase
{
	int32 ExecuteCallCount = 0;
	UMassTestProcessorBase* ObserverProcessor = nullptr;

	virtual bool SetUp() override
	{
		if (!FEntityTestBase::SetUp())
		{
			return false;
		}
		ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		// Override ExecutionFunction so we count raw Execute() invocations, not per-chunk calls.
		ObserverProcessor->ExecutionFunction = [ExecuteCountPtr = &ExecuteCallCount](FMassEntityManager&, FMassExecutionContext&)
		{
			++(*ExecuteCountPtr);
		};
		return true;
	}

	virtual bool InstantTest() override
	{
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		// Register the same processor instance for both Float and Int addition.
		ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);
		ObserverManager.AddObserverInstance(FTestFragment_Int::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

		// Moving EmptyArchetype -> FloatsIntsArchetype adds both Float and Int simultaneously.
		const FMassEntityHandle Entity = EntityManager->CreateEntity(EmptyArchetype);
		EntityManager->MoveEntityToAnotherArchetype(Entity, FloatsIntsArchetype);

		AITEST_EQUAL(TEXT("Observer Execute should fire exactly once even though two observed types were added simultaneously"), ExecuteCallCount, 1);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeObserver_DeduplicationTwoTypes, "System.Mass.Observer.CompositeObserver.DeduplicationTwoTypes");

/**
 * Registers the same processor for Float Add and Int Add, but the composition change only touches Float
 * (entity already owns Int). Verifies that Execute fires exactly once — once for the Float pipeline,
 * and not additionally for Int (which is absent from ObservedTypesOverlap).
 */
struct FCompositeObserver_PartialTypeMatch : FEntityTestBase
{
	int32 ExecuteCallCount = 0;
	UMassTestProcessorBase* ObserverProcessor = nullptr;

	virtual bool SetUp() override
	{
		if (!FEntityTestBase::SetUp())
		{
			return false;
		}
		ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		ObserverProcessor->ExecutionFunction = [ExecuteCountPtr = &ExecuteCallCount](FMassEntityManager&, FMassExecutionContext&)
		{
			++(*ExecuteCountPtr);
		};
		return true;
	}

	virtual bool InstantTest() override
	{
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestFragment_Float::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);
		ObserverManager.AddObserverInstance(FTestFragment_Int::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

		// Entity starts in IntsArchetype (already has Int). Moving to FloatsIntsArchetype only adds Float,
		// so only the Float pipeline is visited in HandleElementsImpl.
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 1, Entities);
		AITEST_EQUAL(TEXT("Observer Execute should fire exactly once, for the initial type"), ExecuteCallCount, 1);
		ExecuteCallCount = 0;

		EntityManager->MoveEntityToAnotherArchetype(Entities[0], FloatsIntsArchetype);

		AITEST_EQUAL(TEXT("Observer Execute should fire exactly once, for the single new type"), ExecuteCallCount, 1);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeObserver_PartialTypeMatch, "System.Mass.Observer.CompositeObserver.PartialTypeMatch");

/**
 * Creates a UMassTestCompositeObserverProcessor, sets ObservedTypes = {Float, Int} and
 * ObservedOperations = Add, then registers it via the generalized
 * AddObserverInstance(UMassObserverProcessor*) API. Verifies that the processor ends up in
 * both type pipelines and that Execute fires exactly once when both types are added simultaneously.
 */
struct FCompositeObserver_BaseApiAddObserver : FEntityTestBase
{
	UMassTestCompositeObserverProcessor* CompositeProcessor = nullptr;

	virtual bool SetUp() override
	{
		if (!FEntityTestBase::SetUp())
		{
			return false;
		}
		CompositeProcessor = NewTestProcessor<UMassTestCompositeObserverProcessor>(EntityManager);
		CompositeProcessor->SetObservedTypesForTest({ FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() });
		CompositeProcessor->SetObservedOperationsForTest(EMassObservedOperationFlags::Add);
		return true;
	}

	virtual bool InstantTest() override
	{
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		// Register via the generalized base class overload.
		ObserverManager.AddObserverInstance(CompositeProcessor);

		// Adding both Float and Int simultaneously should trigger exactly one Execute.
		const FMassEntityHandle Entity = EntityManager->CreateEntity(EmptyArchetype);
		EntityManager->MoveEntityToAnotherArchetype(Entity, FloatsIntsArchetype);

		AITEST_EQUAL(TEXT("Composite observer registered via base API should execute exactly once"), CompositeProcessor->ExecuteCallCount, 1);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeObserver_BaseApiAddObserver, "System.Mass.Observer.CompositeObserver.BaseApiAddObserver");

/**
 * Verifies that RemoveObserverInstance(UMassObserverProcessor*) unregisters the multi-type processor
 * from all type pipelines it was registered in, so that subsequent composition changes no longer
 * trigger Execute.
 */
struct FCompositeObserver_BaseApiRemoveObserver : FEntityTestBase
{
	UMassTestCompositeObserverProcessor* CompositeProcessor = nullptr;

	virtual bool SetUp() override
	{
		if (!FEntityTestBase::SetUp())
		{
			return false;
		}
		CompositeProcessor = NewTestProcessor<UMassTestCompositeObserverProcessor>(EntityManager);
		CompositeProcessor->SetObservedTypesForTest({ FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() });
		CompositeProcessor->SetObservedOperationsForTest(EMassObservedOperationFlags::Add);
		return true;
	}

	virtual bool InstantTest() override
	{
		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(CompositeProcessor);

		// First move: observer should fire.
		const FMassEntityHandle Entity1 = EntityManager->CreateEntity(EmptyArchetype);
		EntityManager->MoveEntityToAnotherArchetype(Entity1, FloatsIntsArchetype);
		AITEST_EQUAL(TEXT("Observer should fire once before removal"), CompositeProcessor->ExecuteCallCount, 1);

		// Remove from all type pipelines via the base API.
		ObserverManager.RemoveObserverInstance(CompositeProcessor);

		// Second move: observer should NOT fire.
		const FMassEntityHandle Entity2 = EntityManager->CreateEntity(EmptyArchetype);
		EntityManager->MoveEntityToAnotherArchetype(Entity2, FloatsIntsArchetype);
		AITEST_EQUAL(TEXT("Observer should not fire after removal"), CompositeProcessor->ExecuteCallCount, 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeObserver_BaseApiRemoveObserver, "System.Mass.Observer.CompositeObserver.BaseApiRemoveObserver");

/**
 * Verifies that GetObservedTypes() on UMassTestCompositeObserverProcessor returns exactly the types
 * that were configured via SetObservedTypesForTest().
 */
struct FCompositeObserver_GetObservedTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		UMassTestCompositeObserverProcessor* Proc = NewTestProcessor<UMassTestCompositeObserverProcessor>(EntityManager);

		AITEST_EQUAL(TEXT("Freshly created composite observer should have no observed types"), Proc->GetObservedTypes().Num(), 0);

		Proc->SetObservedTypesForTest({ FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct() });

		AITEST_EQUAL(TEXT("Composite observer should report 3 observed types"), Proc->GetObservedTypes().Num(), 3);
		AITEST_TRUE(TEXT("First observed type should be Float"), Proc->GetObservedTypes()[0] == FTestFragment_Float::StaticStruct());
		AITEST_TRUE(TEXT("Second observed type should be Int"), Proc->GetObservedTypes()[1] == FTestFragment_Int::StaticStruct());
		AITEST_TRUE(TEXT("Third observed type should be Tag_A"), Proc->GetObservedTypes()[2] == FTestTag_A::StaticStruct());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCompositeObserver_GetObservedTypes, "System.Mass.Observer.CompositeObserver.GetObservedTypes");

/**
 * Verifies that BatchAddSharedFragmentsForEntities fires the AddElement observer for the shared fragment type.
 */
struct FBatchAddSharedFragment_ObserverFires : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> AffectedEntities;
		UMassTestProcessorBase* ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		ObserverProcessor->EntityQuery.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadOnly);
		ObserverProcessor->ForEachEntityChunkExecutionFunction = [&AffectedEntities](FMassExecutionContext& Context)
		{
			AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		};

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestSharedFragment_Int::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 5, Entities);

		FTestSharedFragment_Int FragmentInstance;
		FragmentInstance.Value = 42;
		const FSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateSharedFragment(FragmentInstance);
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(SharedFragmentInstance);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates, EntityCollections);
		EntityManager->BatchAddSharedFragmentsForEntities(EntityCollections, SharedValues);

		AITEST_EQUAL(TEXT("Observer should fire for all entities that received the shared fragment"), AffectedEntities.Num(), Entities.Num());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBatchAddSharedFragment_ObserverFires, "System.Mass.Observer.SharedFragment.BatchAddFiresObserver");

/**
 * Verifies that BatchAddSharedFragmentsForEntities fires the AddElement observer for a const shared fragment type.
 */
struct FBatchAddConstSharedFragment_ObserverFires : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FMassEntityHandle> AffectedEntities;
		UMassTestProcessorBase* ObserverProcessor = NewTestProcessor<UMassTestProcessorBase>(EntityManager);
		ObserverProcessor->EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		ObserverProcessor->EntityQuery.AddConstSharedRequirement<FTestConstSharedFragment_Int>();
		ObserverProcessor->ForEachEntityChunkExecutionFunction = [&AffectedEntities](FMassExecutionContext& Context)
		{
			AffectedEntities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
		};

		FMassObserverManager& ObserverManager = EntityManager->GetObserverManager();
		ObserverManager.AddObserverInstance(FTestConstSharedFragment_Int::StaticStruct(), EMassObservedOperationFlags::Add, ObserverProcessor);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 5, Entities);

		FTestConstSharedFragment_Int FragmentInstance;
		FragmentInstance.Value = 99;
		const FConstSharedStruct ConstSharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(ConstSharedFragmentInstance);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates, EntityCollections);
		EntityManager->BatchAddSharedFragmentsForEntities(EntityCollections, SharedValues);

		AITEST_EQUAL(TEXT("Observer should fire for all entities that received the const shared fragment"), AffectedEntities.Num(), Entities.Num());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FBatchAddConstSharedFragment_ObserverFires, "System.Mass.Observer.SharedFragment.BatchAddConstSharedFiresObserver");

} // UE::Mass::Test::Observers

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
