// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"

#include "Engine/World.h"
#include "MassEntityManager.h"
#include "MassProcessingTypes.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassArchetypeData.h"

#define LOCTEXT_NAMESPACE "MassTest"

namespace FMassQueryTest
{

struct FQueryTest_ParallelForBasic : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();

		const int32 NumEntitiesToCreate = 32 * EntitiesPerChunk;
		TArray<FMassEntityHandle> CreatedEntities;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumEntitiesToCreate, CreatedEntities);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);

		std::atomic<int32> EntitiesProcessed = 0;
		std::atomic<int32> ConcurrentAccess = 0;
		FCriticalSection ParallelDetectionCS;

		FMassExecutionContext Context = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0.f);
		Query.ParallelForEachEntityChunk(Context, [&ParallelDetectionCS, &EntitiesProcessed, &ConcurrentAccess](FMassExecutionContext& Context)
			{
				EntitiesProcessed += Context.GetNumEntities();

				if (ParallelDetectionCS.TryLock() == false)
				{
					++ConcurrentAccess;
					return;
				}
				FPlatformProcess::Sleep(0.01f);
			}, FMassEntityQuery::EParallelExecutionFlags::Force);
		
		AITEST_EQUAL("All created entites should have been processed at this point", EntitiesProcessed.load(), NumEntitiesToCreate);
		AITEST_TRUE("Some work is expected to be executed in parallel", ConcurrentAccess.load() > 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ParallelForBasic, "System.Mass.Query.ParallelForBasic");

struct FQueryTest_ParallelForCollection : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 NumEntitiesToCreate = 1000;
		TArray<FMassEntityHandle> CreatedEntities;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumEntitiesToCreate, CreatedEntities);

		TArray<FMassEntityHandle> EntitiesToProcess;
		EntitiesToProcess.Reserve(NumEntitiesToCreate);

		FRandomStream RandomStream(/*Seed=*/1);
		for (FMassEntityHandle& EntityHandle : CreatedEntities)
		{
			if (RandomStream.FRand() < 0.5)
			{
				EntitiesToProcess.Add(EntityHandle);
			}
		}
		ensure(EntitiesToProcess.Num() > 0);
		const FMassArchetypeEntityCollection EntityCollection(FloatsArchetype, EntitiesToProcess, FMassArchetypeEntityCollection::NoDuplicates);
		const int32 NumJobs = EntityCollection.GetRanges().Num();

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);

		std::atomic<int32> EntitiesProcessed = 0;
		std::atomic<int32> ConcurrentAccess = 0;
		std::atomic<int32> JobsExecuted = 0;
		FCriticalSection ParallelDetectionCS;

		FMassExecutionContext Context = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0.f);
		Context.SetEntityCollection(EntityCollection);

		Query.ParallelForEachEntityChunk(Context, [&JobsExecuted, &ParallelDetectionCS, &EntitiesProcessed, &ConcurrentAccess](FMassExecutionContext& Context)
			{
				++JobsExecuted;
				EntitiesProcessed += Context.GetNumEntities();

				if (ParallelDetectionCS.TryLock() == false)
				{
					++ConcurrentAccess;
					return;
				}
				FPlatformProcess::Sleep(0.01f);
			}, FMassEntityQuery::EParallelExecutionFlags::Force);

		AITEST_EQUAL("All entities passed for processing should have been counted", EntitiesProcessed.load(), EntitiesToProcess.Num());
		AITEST_EQUAL("The number of jobs should match expectations", NumJobs, JobsExecuted.load());
		AITEST_TRUE("Some work is expected to be executed in parallel", ConcurrentAccess.load() > 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ParallelForCollection, "System.Mass.Query.ParallelForCollection");

#if WITH_MASSENTITY_DEBUG
struct FQueryTest_ParallelCommands : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassArchetypeHandle TargetArchetype = EntityManager->CreateArchetype(FloatsArchetype, { FTestFragment_Tag::StaticStruct() });

		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();

		const int32 NumEntitiesToCreate = 32 * EntitiesPerChunk;
		TArray<FMassEntityHandle> CreatedEntities;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumEntitiesToCreate, CreatedEntities);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
		Query.SetParallelCommandBufferEnabled(true);

		const int32 OriginalEntityCount = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
		AITEST_EQUAL("Target archetype should only contain freshly created entities", OriginalEntityCount, NumEntitiesToCreate);

		FMassExecutionContext Context = EntityManager->CreateExecutionContext(/*DeltaSeconds=*/0.f);
		Query.ParallelForEachEntityChunk(Context, [](FMassExecutionContext& Context)
			{
				Context.Defer().PushCommand<FMassCommandAddTags<FTestFragment_Tag>>(Context.GetEntities());
			}, FMassEntityQuery::EParallelExecutionFlags::Force);

		const int32 OriginalArchetypeCountAfterMove = EntityManager->DebugGetArchetypeEntitiesCount(FloatsArchetype);
		AITEST_EQUAL("Target archetype should be empty after the move", OriginalArchetypeCountAfterMove, 0);
		const int32 TargetArchetypeCountAfterMove = EntityManager->DebugGetArchetypeEntitiesCount(TargetArchetype);
		AITEST_EQUAL("All entities are expected to be moved over to the target archetype", OriginalEntityCount, TargetArchetypeCountAfterMove);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ParallelCommands, "System.Mass.Query.ParallelForCommands");
#endif // WITH_MASSENTITY_DEBUG

} // FMassQueryTest

#undef LOCTEXT_NAMESPACE

