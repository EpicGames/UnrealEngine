// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassProcessingPhaseManager.h"
#include "MassEntityTestTypes.h"
#include "MassProcessingPhaseManager.h"
#include "Engine/World.h"

#define LOCTEXT_NAMESPACE "MassTest"


UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// FProcessingPhasesTestBase
//----------------------------------------------------------------------//
FProcessingPhasesTestBase::FProcessingPhasesTestBase()
{
	bMakeWorldEntityManagersOwner = true;
}

bool FProcessingPhasesTestBase::SetUp()
{
	if (Super::SetUp())
	{
		PhaseManager = MakeShareable(new UE::Mass::Testing::FMassTestProcessingPhaseManager());

		FMassEntityManagerStorageInitParams InitializationParams;
#if WITH_MASS_CONCURRENT_RESERVE

		InitializationParams.Emplace<FMassEntityManager_InitParams_Concurrent>();
#else
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InitializationParams.Emplace<FMassEntityManager_InitParams_SingleThreaded>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_MASS_CONCURRENT_RESERVE

		EntityManager->Initialize(InitializationParams);

		if (PopulatePhasesConfig())
		{
			TickIndex = -1;

			World = FAITestHelpers::GetWorld();
			check(World);

			PhaseManager->Initialize(*World, PhasesConfig);

			PhaseManager->Start(EntityManager);

			return true;
		}
	}

	return false;
}

bool FProcessingPhasesTestBase::Update() 
{
	if (CompletionEvent.IsValid())
	{
		CompletionEvent->Wait();
	}

	for (int PhaseIndex = 0; PhaseIndex < (int)EMassProcessingPhase::MAX; ++PhaseIndex)
	{
		const FGraphEventArray Prerequisites = { CompletionEvent };
		CompletionEvent = TGraphTask<UE::Mass::Testing::FMassTestPhaseTickTask>::CreateTask(&Prerequisites)
			.ConstructAndDispatchWhenReady(PhaseManager.ToSharedRef(), EMassProcessingPhase(PhaseIndex), DeltaTime);
	}

	++TickIndex;
	return false;
}

void FProcessingPhasesTestBase::TearDown()
{
	if (CompletionEvent.IsValid())
	{
		CompletionEvent->Wait();
	}

	PhaseManager->Stop();
	PhaseManager = nullptr;
	Super::TearDown();
}

void FProcessingPhasesTestBase::VerifyLatentResults()
{
	// we need to make sure all the phases are done before attempting to test results
	if (CompletionEvent.IsValid())
	{
		CompletionEvent->Wait();
	}
}


//----------------------------------------------------------------------//
// FMassProcessingPhasesTest
//----------------------------------------------------------------------//
namespace FMassProcessingPhasesTest
{

/** this test is here to make sure that the set up that other tests rely on actually works, i.e. the phases are getting ticked at all*/
struct FTestSetupTest : FProcessingPhasesTestBase
{
	using Super = FProcessingPhasesTestBase;

	virtual bool SetUp() override
	{
		if (Super::SetUp())
		{
			UMassTestStaticCounterProcessor::StaticCounter = -1;
			return true;
		}
		return false;
	}

	virtual bool PopulatePhasesConfig()
	{
		PhasesConfig[0].ProcessorCDOs.Add(GetMutableDefault<UMassTestStaticCounterProcessor>());
		return true;
	}

	virtual bool Update() override
	{
		Super::Update();
		return TickIndex >= 3;
	}

	virtual void VerifyLatentResults() override
	{
		Super::VerifyLatentResults();
		AITEST_EQUAL_LATENT("Expecting the UMassTestStaticCounterProcessor getting ticked as many times as the test ticked", UMassTestStaticCounterProcessor::StaticCounter, TickIndex);
	}
};
IMPLEMENT_AI_LATENT_TEST(FTestSetupTest, "System.Mass.ProcessingPhases.SetupTest");

} // FMassProcessingPhasesTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE

