// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassProcessorTest
{

template<typename TProcessor>
int32 SimpleProcessorRun(FMassEntityManager& EntityManager)
{
	int32 EntityProcessedCount = 0;
	TProcessor* Processor = NewObject<TProcessor>();
	Processor->CallInitialize(GetTransientPackage(), EntityManager.AsShared());
	Processor->ForEachEntityChunkExecutionFunction = [&EntityProcessedCount](FMassExecutionContext& Context) 
	{
		EntityProcessedCount += Context.GetNumEntities();
	};

	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::Run(*Processor, ProcessingContext);

	return EntityProcessedCount;
}

struct FProcessorTest_NoEntities : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		
		int32 EntityProcessedCount = SimpleProcessorRun<UMassTestProcessor_Floats>(*EntityManager);
		AITEST_EQUAL("No entities have been created yet so the processor shouldn't do any work.", EntityProcessedCount, 0);
				
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessorTest_NoEntities, "System.Mass.Processor.NoEntities");


struct FProcessorTest_NoMatchingEntities : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(IntsArchetype, 100, EntitiesCreated);
		int32 EntityProcessedCount = SimpleProcessorRun<UMassTestProcessor_Floats>(*EntityManager);
		AITEST_EQUAL("No matching entities have been created yet so the processor shouldn't do any work.", EntityProcessedCount, 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessorTest_NoMatchingEntities, "System.Mass.Processor.NoMatchingEntities");

struct FProcessorTest_OneMatchingArchetype : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 EntitiesToCreate = 100;
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToCreate, EntitiesCreated);
		int32 EntityProcessedCount = SimpleProcessorRun<UMassTestProcessor_Floats>(*EntityManager);
		AITEST_EQUAL("No matching entities have been created yet so the processor shouldn't do any work.", EntityProcessedCount, EntitiesToCreate);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessorTest_OneMatchingArchetype, "System.Mass.Processor.OneMatchingArchetype");


struct FProcessorTest_MultipleMatchingArchetype : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 EntitiesToCreate = 100;
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(FloatsArchetype, EntitiesToCreate, EntitiesCreated);
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, EntitiesToCreate, EntitiesCreated);
		EntityManager->BatchCreateEntities(IntsArchetype, EntitiesToCreate, EntitiesCreated);
		// note that only two of these archetypes match 
		int32 EntityProcessedCount = SimpleProcessorRun<UMassTestProcessor_Floats>(*EntityManager);
		AITEST_EQUAL("Two of given three archetypes should match.", EntityProcessedCount, EntitiesToCreate * 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessorTest_MultipleMatchingArchetype, "System.Mass.Processor.MultipleMatchingArchetype");


struct FProcessorTest_Requirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		// add an array to the configurable ExecutionFunction an

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FProcessorTest_Requirements, "System.Mass.Processor.Requirements");


} // FMassProcessorTestTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE

