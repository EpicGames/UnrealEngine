// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassProcessingTypes.h"
#include "MassEntityTestTypes.h"
#include "MassEntityUtils.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "MassQueryExecutor.h"
#include "HAL/PlatformTime.h"


#define LOCTEXT_NAMESPACE "MassTest"

DEFINE_LOG_CATEGORY_STATIC(LogMassPerfTest, Log, All);

UE_DISABLE_OPTIMIZATION_SHIP

void UMassTestProcessorAutoExecuteQueryComparison::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Large>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Array>(EMassFragmentAccess::ReadOnly);
}

void UMassTestProcessorAutoExecuteQueryComparison::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		const TConstArrayView<FTestFragment_Float> TestFloat = Context.GetFragmentView<FTestFragment_Float>();
		const TArrayView<FTestFragment_Int> TestInt = Context.GetMutableFragmentView<FTestFragment_Int>();
		const TConstArrayView<FTestFragment_Bool> TestBool = Context.GetFragmentView<FTestFragment_Bool>();
		const TConstArrayView<FTestFragment_Large> TestLarge = Context.GetFragmentView<FTestFragment_Large>();
		const TConstArrayView<FTestFragment_Array> TestArray = Context.GetFragmentView<FTestFragment_Array>();

		for (uint32 EntityIndex : Context.CreateEntityIterator())
		{
			int32& TestIntVal = TestInt[EntityIndex].Value;
			if (TestFloat[EntityIndex].Value > 0.0f)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestBool[EntityIndex].bValue)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestLarge[EntityIndex].Value[0] > 0)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}

			if (TestArray[EntityIndex].Value.Num() > 0)
			{
				++TestIntVal;
			}
			else
			{
				--TestIntVal;
			}
		}
	});
}

void UMassTestProcessorAutoExecuteQueryComparison_Parallel::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Large>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTestFragment_Array>(EMassFragmentAccess::ReadOnly);
}

void UMassTestProcessorAutoExecuteQueryComparison_Parallel::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ParallelForEachEntityChunk(Context, [](FMassExecutionContext& Context)
		{
			const TConstArrayView<FTestFragment_Float> TestFloat = Context.GetFragmentView<FTestFragment_Float>();
			const TArrayView<FTestFragment_Int> TestInt = Context.GetMutableFragmentView<FTestFragment_Int>();
			const TConstArrayView<FTestFragment_Bool> TestBool = Context.GetFragmentView<FTestFragment_Bool>();
			const TConstArrayView<FTestFragment_Large> TestLarge = Context.GetFragmentView<FTestFragment_Large>();
			const TConstArrayView<FTestFragment_Array> TestArray = Context.GetFragmentView<FTestFragment_Array>();

			for (uint32 EntityIndex : Context.CreateEntityIterator())
			{
				int32& TestIntVal = TestInt[EntityIndex].Value;
				if (TestFloat[EntityIndex].Value > 0.0f)
				{
					++TestIntVal;
				}
				else
				{
					--TestIntVal;
				}

				if (TestBool[EntityIndex].bValue)
				{
					++TestIntVal;
				}
				else
				{
					--TestIntVal;
				}

				if (TestLarge[EntityIndex].Value[0] > 0)
				{
					++TestIntVal;
				}
				else
				{
					--TestIntVal;
				}

				if (TestArray[EntityIndex].Value.Num() > 0)
				{
					++TestIntVal;
				}
				else
				{
					--TestIntVal;
				}
			}
		});
}

namespace FMassQueryExecutorTest
{

	struct FTestQueryExecutor_LoadTestSetup : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Bool>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Large>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Array>,
			UE::Mass::FMutableSubsystemAccess<UMassTestWorldSubsystem>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				UMassTestWorldSubsystem& TestSubSys = Data.template Get<UMassTestWorldSubsystem>();
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					Data.template Get<FTestFragment_Int>()[EntityIndex].Value = 0;
					Data.template Get<FTestFragment_Float>()[EntityIndex].Value = FMath::FRandRange(-1.0f, 1.0f);
					Data.template Get<FTestFragment_Bool>()[EntityIndex].bValue = FMath::RandBool();
					Data.template Get<FTestFragment_Large>()[EntityIndex].Value[0] = FMath::RandBool() ? 1 : 0;
					Data.template Get<FTestFragment_Array>()[EntityIndex].Value.SetNum(FMath::RandBool() ? 1 : 0);
					TestSubSys.Write(0);
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTestReset : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					Data.template Get<FTestFragment_Int>()[EntityIndex].Value = 0;
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTestValidate : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Int>
		> Accessors{ *this };

		std::atomic<int64> Sum = 0;

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntityChunk(Context, Accessors, [&Sum = Sum](FMassExecutionContext& Context, auto& Data)
			{
				int64 ChunkSum = 0;
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					ChunkSum += Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
				}
				Sum += ChunkSum;
			});
		}
	};

	struct FTestQueryExecutor_LoadTest : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Bool>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Large>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Array>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					int32& TestInt = Data.template Get<FTestFragment_Int>()[EntityIndex].Value;

					if (Data.template Get<FTestFragment_Float>()[EntityIndex].Value > 0.0f)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}

					if (Data.template Get<FTestFragment_Bool>()[EntityIndex].bValue)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}

					if (Data.template Get<FTestFragment_Large>()[EntityIndex].Value[0] > 0)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}

					if (Data.template Get<FTestFragment_Array>()[EntityIndex].Value.Num() > 0)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTest_Parallel : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Bool>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Large>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Array>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					int32& TestInt = Data.template Get<FTestFragment_Int>()[EntityIndex].Value;

					if (Data.template Get<FTestFragment_Float>()[EntityIndex].Value > 0.0f)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}

					if (Data.template Get<FTestFragment_Bool>()[EntityIndex].bValue)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}

					if (Data.template Get<FTestFragment_Large>()[EntityIndex].Value[0] > 0)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}

					if (Data.template Get<FTestFragment_Array>()[EntityIndex].Value.Num() > 0)
					{
						++TestInt;
					}
					else
					{
						--TestInt;
					}
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTest_ByEntity : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Bool>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Large>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Array>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ForEachEntity(Context, Accessors, [](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
			{
				int32& TestInt = Data.template Get<FTestFragment_Int>()[EntityIndex].Value;

				if (Data.template Get<FTestFragment_Float>()[EntityIndex].Value > 0.0f)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}

				if (Data.template Get<FTestFragment_Bool>()[EntityIndex].bValue)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}

				if (Data.template Get<FTestFragment_Large>()[EntityIndex].Value[0] > 0)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}

				if (Data.template Get<FTestFragment_Array>()[EntityIndex].Value.Num() > 0)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTest_ByEntity_Parallel : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Bool>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Large>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Array>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntity(Context, Accessors, [](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
			{
				int32& TestInt = Data.template Get<FTestFragment_Int>()[EntityIndex].Value;

				if (Data.template Get<FTestFragment_Float>()[EntityIndex].Value > 0.0f)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}

				if (Data.template Get<FTestFragment_Bool>()[EntityIndex].bValue)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}

				if (Data.template Get<FTestFragment_Large>()[EntityIndex].Value[0] > 0)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}

				if (Data.template Get<FTestFragment_Array>()[EntityIndex].Value.Num() > 0)
				{
					++TestInt;
				}
				else
				{
					--TestInt;
				}
			});
		}
	};

	struct FTestQueryExecutor_AnyTag : public UE::Mass::FQueryExecutor
	{
		int32 EntityCount = 0;

		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ForEachEntity(Context, Accessors, [this](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
			{
				++EntityCount;
			});
		}
	};

	struct FTestQueryExecutor_NeedTagA : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMassTagRequired<FTestTag_A>
		> Accessors{ *this };

		int32 EntityCount = 0;

		virtual void Execute(FMassExecutionContext& Context)
		{
			ForEachEntity(Context, Accessors, [this](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
			{
				++EntityCount;
			});
		}
	};

	struct FTestQueryExecutor_BlockTagB : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMassTagBlocked<FTestTag_B>
		> Accessors{ *this };

		int32 EntityCount = 0;

		virtual void Execute(FMassExecutionContext& Context)
		{
			ForEachEntity(Context, Accessors, [this](FMassExecutionContext& Context, auto& Data, uint32 EntityIndex)
			{
				++EntityCount;
			});
		}
	};

	struct FQueryExecutor_IteratorConsistency : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			CA_ASSUME(EntityManager);

			const int32 EntityCountNoTag = 7;
			const int32 EntityCountA = 9;
			const int32 EntityCountB = 13;
			const int32 EntityCountAB = 17;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountNoTag, Entities);

			Entities.Reset();
			EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountA, Entities);
			for (FMassEntityHandle& Entity : Entities)
			{
				EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
			}

			Entities.Reset();
			EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountB, Entities);
			for (FMassEntityHandle& Entity : Entities)
			{
				EntityManager->AddTagToEntity(Entity, FTestTag_B::StaticStruct());
			}

			Entities.Reset();
			EntityManager->BatchCreateEntities(FloatsArchetype, {}, EntityCountAB, Entities);
			for (FMassEntityHandle& Entity : Entities)
			{
				EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
				EntityManager->AddTagToEntity(Entity, FTestTag_B::StaticStruct());
			}

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_AnyTag = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_AnyTag);
			TSharedPtr<FTestQueryExecutor_AnyTag> TestQuery_AnyTag = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_AnyTag>(Processor_AnyTag->EntityQuery, Processor_AnyTag);
			Processor_AnyTag->SetAutoExecuteQuery(TestQuery_AnyTag);
			Processor_AnyTag->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_TagA = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_TagA);
			TSharedPtr<FTestQueryExecutor_NeedTagA> TestQuery_TagA = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_NeedTagA>(Processor_TagA->EntityQuery, Processor_TagA);
			Processor_TagA->SetAutoExecuteQuery(TestQuery_TagA);
			Processor_TagA->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_TagB = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_TagB);
			TSharedPtr<FTestQueryExecutor_BlockTagB> TestQuery_TagB = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_BlockTagB>(Processor_TagB->EntityQuery, Processor_TagB);
			Processor_TagB->SetAutoExecuteQuery(TestQuery_TagB);
			Processor_TagB->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TArray<TObjectPtr<UMassProcessor>> Processors;
			Processors.Add(Processor_AnyTag);
			Processors.Add(Processor_TagA);
			Processors.Add(Processor_TagB);

			FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);

			UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);

			const int32 AnyCount = TestQuery_AnyTag->EntityCount;
			const int32 ExpectedAnyCount = EntityCountNoTag + EntityCountA + EntityCountB + EntityCountAB;
			const int32 ACount = TestQuery_TagA->EntityCount;
			const int32 ExpectedACount = EntityCountA + EntityCountAB;
			const int32 BCount = TestQuery_TagB->EntityCount;
			const int32 ExpectedBCount = EntityCountNoTag + EntityCountA;

			AITEST_EQUAL(TEXT("Any Tag Entities Processed"), AnyCount, ExpectedAnyCount);
			AITEST_EQUAL(TEXT("Require TagA Entities Processed"), ACount, ExpectedACount);
			AITEST_EQUAL(TEXT("Blocked TagB Entities Processed"), BCount, ExpectedBCount);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FQueryExecutor_IteratorConsistency, "System.Mass.Processor.AutoExecuteQuery.IteratorConsistency");

	struct FTestQueryExecutor_LoadTest_Float : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FConstFragmentAccess<FTestFragment_Float>,
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					if (Data.template Get<FTestFragment_Float>()[EntityIndex].Value > 0.0f)
					{
						++Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
					else
					{
						--Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTest_Bool : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Bool>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					if (Data.template Get<FTestFragment_Bool>()[EntityIndex].bValue)
					{
						++Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
					else
					{
						--Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTest_Large : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Large>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					int32& TestInt = Data.template Get<FTestFragment_Int>()[EntityIndex].Value;

					if (Data.template Get<FTestFragment_Large>()[EntityIndex].Value[0] > 0)
					{
						++Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
					else
					{
						--Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
				}
			});
		}
	};

	struct FTestQueryExecutor_LoadTest_Array : public UE::Mass::FQueryExecutor
	{
		UE::Mass::FQueryDefinition<
			UE::Mass::FMutableFragmentAccess<FTestFragment_Int>,
			UE::Mass::FConstFragmentAccess<FTestFragment_Array>
		> Accessors{ *this };

		virtual void Execute(FMassExecutionContext& Context)
		{
			ParallelForEachEntityChunk(Context, Accessors, [](FMassExecutionContext& Context, auto& Data)
			{
				for (uint32 EntityIndex : Context.CreateEntityIterator())
				{
					if (Data.template Get<FTestFragment_Array>()[EntityIndex].Value.Num() > 0)
					{
						++Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
					else
					{
						--Data.template Get<FTestFragment_Int>()[EntityIndex].Value;
					}
				}
			});
		}
	};

	struct FQueryExecutor_LoadTest : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			CA_ASSUME(EntityManager);

			constexpr uint32 NumTags = 5;
			const UScriptStruct* Tags[NumTags] = {
				FTestFragment_Tag::StaticStruct(),
				FTestTag_A::StaticStruct(),
				FTestTag_B::StaticStruct(),
				FTestTag_C::StaticStruct(),
				FTestTag_D::StaticStruct(),
			};

			TArray<UScriptStruct*> Fragments;

			Fragments.Add(FTestFragment_Float::StaticStruct());
			Fragments.Add(FTestFragment_Int::StaticStruct());
			Fragments.Add(FTestFragment_Bool::StaticStruct());
			Fragments.Add(FTestFragment_Large::StaticStruct());
			Fragments.Add(FTestFragment_Array::StaticStruct());

			FMassArchetypeHandle LoadTestArchetype = EntityManager->CreateArchetype(Fragments);

			TArray<FMassEntityHandle> Entities;

			auto CreateEntities = [&](uint32 EntityCount)
			{
				EntityManager->BatchDestroyEntities(Entities);
				Entities.Reset(EntityCount);
				EntityManager->BatchCreateEntities(LoadTestArchetype, {}, EntityCount, Entities);

				int32 TagMask = 0;
				for (FMassEntityHandle& Entity : Entities)
				{
					// maximize entity fragmentation to test worst-case scenario

					for (int32 BitIndex = 0; BitIndex < NumTags; ++BitIndex)
					{
						if ((TagMask & (1 << BitIndex)) != 0)
						{
							EntityManager->AddTagToEntity(Entity, Tags[BitIndex]);
						}
					}

					if (++TagMask >= (1 << NumTags))
					{
						TagMask = 0;
					}
				}
			};

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_LoadTest_Parallel = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_LoadTest_Parallel);
			TSharedPtr<FTestQueryExecutor_LoadTest_Parallel> TestQuery_LoadTest_Parallel = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest_Parallel>(Processor_LoadTest_Parallel->EntityQuery, Processor_LoadTest_Parallel);
			Processor_LoadTest_Parallel->SetAutoExecuteQuery(TestQuery_LoadTest_Parallel);
			Processor_LoadTest_Parallel->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_LoadTest_ByEntity_Parallel = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_LoadTest_ByEntity_Parallel);
			TSharedPtr<FTestQueryExecutor_LoadTest_ByEntity_Parallel> TestQuery_LoadTest_ByEntity_Parallel = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest_ByEntity_Parallel>(Processor_LoadTest_ByEntity_Parallel->EntityQuery, Processor_LoadTest_ByEntity_Parallel);
			Processor_LoadTest_ByEntity_Parallel->SetAutoExecuteQuery(TestQuery_LoadTest_ByEntity_Parallel);
			Processor_LoadTest_ByEntity_Parallel->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_LoadTest = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_LoadTest);
			TSharedPtr<FTestQueryExecutor_LoadTest> TestQuery_LoadTest = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest>(Processor_LoadTest->EntityQuery, Processor_LoadTest);
			Processor_LoadTest->SetAutoExecuteQuery(TestQuery_LoadTest);
			Processor_LoadTest->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_LoadTest_ByEntity = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_LoadTest_ByEntity);
			TSharedPtr<FTestQueryExecutor_LoadTest_ByEntity> TestQuery_LoadTest_ByEntity = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest_ByEntity>(Processor_LoadTest_ByEntity->EntityQuery, Processor_LoadTest_ByEntity);
			Processor_LoadTest_ByEntity->SetAutoExecuteQuery(TestQuery_LoadTest_ByEntity);
			Processor_LoadTest_ByEntity->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_Setup = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_Setup);
			TSharedPtr<FTestQueryExecutor_LoadTestSetup> TestQuery_Setup = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTestSetup>(Processor_Setup->EntityQuery, Processor_Setup);
			Processor_Setup->SetAutoExecuteQuery(TestQuery_Setup);
			Processor_Setup->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef()); 

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_Reset = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_Reset);
			TSharedPtr<FTestQueryExecutor_LoadTestReset> TestQuery_Reset = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTestReset>(Processor_Reset->EntityQuery, Processor_Reset);
			Processor_Reset->SetAutoExecuteQuery(TestQuery_Reset);
			Processor_Reset->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_Validate = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_Validate);
			TSharedPtr<FTestQueryExecutor_LoadTestValidate> TestQuery_Validate = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTestValidate>(Processor_Validate->EntityQuery, Processor_Validate);
			Processor_Validate->SetAutoExecuteQuery(TestQuery_Validate);
			Processor_Validate->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_Float = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_Float);
			TSharedPtr<FTestQueryExecutor_LoadTest_Float> TestQuery_Float = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest_Float>(Processor_Float->EntityQuery, Processor_Float);
			Processor_Float->SetAutoExecuteQuery(TestQuery_Float);
			Processor_Float->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_Bool = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_Bool);
			TSharedPtr<FTestQueryExecutor_LoadTest_Bool> TestQuery_Bool = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest_Bool>(Processor_Bool->EntityQuery, Processor_Bool);
			Processor_Bool->SetAutoExecuteQuery(TestQuery_Bool);
			Processor_Bool->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_Large = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_Large);
			TSharedPtr<FTestQueryExecutor_LoadTest_Large> TestQuery_Large = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest_Large>(Processor_Large->EntityQuery, Processor_Large);
			Processor_Large->SetAutoExecuteQuery(TestQuery_Large);
			Processor_Large->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TObjectPtr<UMassTestProcessorAutoExecuteQuery> Processor_Array = NewObject<UMassTestProcessorAutoExecuteQuery>();
			CA_ASSUME(Processor_Array);
			TSharedPtr<FTestQueryExecutor_LoadTest_Array> TestQuery_Array = UE::Mass::FQueryExecutor::CreateQuery<FTestQueryExecutor_LoadTest_Array>(Processor_Array->EntityQuery, Processor_Array);
			Processor_Array->SetAutoExecuteQuery(TestQuery_Array);
			Processor_Array->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());

			TArray<TObjectPtr<UMassProcessor>> IndividualProcessors_LoadTest;
			IndividualProcessors_LoadTest.Add(Processor_Float);
			IndividualProcessors_LoadTest.Add(Processor_Bool);
			IndividualProcessors_LoadTest.Add(Processor_Large);
			IndividualProcessors_LoadTest.Add(Processor_Array);

			TArray<TObjectPtr<UMassProcessor>> Processors;
			FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);

			UMassTestProcessorAutoExecuteQueryComparison* DefaultProcessor = NewTestProcessor<UMassTestProcessorAutoExecuteQueryComparison>(EntityManager);
			UMassTestProcessorAutoExecuteQueryComparison_Parallel* DefaultProcessor_Parallel = NewTestProcessor<UMassTestProcessorAutoExecuteQueryComparison_Parallel>(EntityManager);

			Processors.Reset();
			Processors.Add(Processor_Setup);
			UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);

			struct TestReturn
			{
				const TCHAR* Name;
				uint64 Sum;
				double Duration;
				double NormalizedTime;
			};

			auto RunTest = [&](TObjectPtr<UMassProcessor> Proc, TArray<TObjectPtr<UMassProcessor>>* ProcArray, uint32 EntityCount, uint32 RunCount, const TCHAR* Name) -> TestReturn
			{
				Processors.Reset();
				if (ProcArray != nullptr)
				{
					Processors.Append(*ProcArray);
				}

				if (Proc)
				{
					Processors.Add(Proc);
				}
			
				// warm the cache with a single untimed iteration
				UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);

				const double StartTime = FPlatformTime::Seconds();
				for (uint32 i = 0; i < RunCount; i++)
				{
					UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);
				}
				const double EndTime = FPlatformTime::Seconds();
				const double Duration = EndTime - StartTime;
				const double NormalizedTime = (Duration) / ((((double)EntityCount)/1000.0) * (((double)RunCount)/1000.0));
				Processors.Reset();
				Processors.Add(Processor_Validate);
				UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);
				uint64 ProcSum = TestQuery_Validate->Sum;
				TestQuery_Validate->Sum = 0;

				Processors.Reset();
				Processors.Add(Processor_Reset);
				UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);

				return { Name, ProcSum, Duration, NormalizedTime };
			};


			auto LogResult = [&](TestReturn& Test, const TestReturn& Baseline)
			{
				const double PercentDiff = ((Test.NormalizedTime - Baseline.NormalizedTime) / Baseline.NormalizedTime) * 100.0;
				if (PercentDiff > 0.0)
				{
					UE_LOGF(LogMassPerfTest, Log, "%ls: %.4fns/Entity (-%.4f%%).", Test.Name, Test.NormalizedTime, PercentDiff);
				}
				else if (PercentDiff <= 0.0)
				{
					UE_LOGF(LogMassPerfTest, Log, "%ls: %.4fns/Entity (+%.4f%%).", Test.Name, Test.NormalizedTime, FMath::Abs(PercentDiff));
				}
			};

			auto LogTime = [&](TestReturn& Test)
			{
				UE_LOGF(LogMassPerfTest, Log, "%ls,%.8f", Test.Name, Test.NormalizedTime);
			};

			auto SortPredicate = [](const TestReturn& A, const TestReturn& B)
			{
				return A.NormalizedTime < B.NormalizedTime;
			};

			auto RunBattery = [&](uint32 EntityCount, uint32 RunCount)
			{
				UE_LOGF(LogMassPerfTest, Log, "\n%d entities:", EntityCount);
				CreateEntities(EntityCount);
				Processors.Reset();
				Processors.Add(Processor_Setup);
				UE::Mass::Executor::RunProcessorsView(Processors, ProcessingContext);
				TArray<TestReturn> Results;
				int32 DefaultIndex = Results.Emplace(RunTest(DefaultProcessor, nullptr, EntityCount, RunCount, TEXT("Default")));
				TestReturn Default = Results[DefaultIndex];
				Results.Emplace(RunTest(nullptr, &IndividualProcessors_LoadTest, EntityCount, RunCount, TEXT("Split_Processing_Parallel")));
				Results.Emplace(RunTest(Processor_LoadTest, nullptr, EntityCount, RunCount, TEXT("QueryExecutor")));
				Results.Emplace(RunTest(Processor_LoadTest_ByEntity, nullptr, EntityCount, RunCount, TEXT("QueryExecutor_ByEntity")));
				Results.Emplace(RunTest(Processor_LoadTest_Parallel, nullptr, EntityCount, RunCount, TEXT("QueryExecutor_Parallel")));
				Results.Emplace(RunTest(Processor_LoadTest_ByEntity_Parallel, nullptr, EntityCount, RunCount, TEXT("QueryExecutor_ByEntity_Parallel")));
				Results.Emplace(RunTest(DefaultProcessor_Parallel, nullptr, EntityCount, RunCount, TEXT("Default_Parallel")));

				for (int32 i = 0; i < Results.Num(); i++)
				{
					FString TestMessage = FString::Format(TEXT("{0} Sum should match Default"), { Results[i].Name});
					AITEST_EQUAL(TestMessage, Results[i].Sum, Default.Sum);
				}


				for (TestReturn& Result : Results)
				{
					LogTime(Result);
				}

				return true;
			};
			
			RunBattery(1, 10000);
			RunBattery(10, 1000);
			RunBattery(100, 100);
			RunBattery(1000, 100);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FQueryExecutor_LoadTest, "System.Mass.Processor.AutoExecuteQuery.LoadTest");

} // FMassQueryExecutorTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
