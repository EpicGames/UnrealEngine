// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "Engine/World.h"
#include "MassEntityManager.h"
#include "MassProcessingContext.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassTypeManager.h"
#include "MassEntityLinkFragments.h"
#include "Algo/Compare.h"

#define LOCTEXT_NAMESPACE "MassTest"


namespace FMassQueryTest
{

struct FQueryTest_ProcessorRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		UMassTestProcessor_Floats* Processor = NewTestProcessor<UMassTestProcessor_Floats>(EntityManager);
		TConstArrayView<FMassFragmentRequirementDescription> Requirements = Processor->EntityQuery.GetFragmentRequirements();
		
		AITEST_TRUE("Query should have extracted some requirements from the given Processor", Requirements.Num() > 0);
		AITEST_TRUE("There should be exactly one requirement", Requirements.Num() == 1);
		AITEST_TRUE("The requirement should be of the Float fragment type", Requirements[0].StructType == FTestFragment_Float::StaticStruct());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ProcessorRequirements, "System.Mass.Query.ProcessorRequiements");


struct FQueryTest_ExplicitRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
				
		FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct()});
		TConstArrayView<FMassFragmentRequirementDescription> Requirements = Query.GetFragmentRequirements();

		AITEST_TRUE("Query should have extracted some requirements from the given Processor", Requirements.Num() > 0);
		AITEST_TRUE("There should be exactly one requirement", Requirements.Num() == 1);
		AITEST_TRUE("The requirement should be of the Float fragment type", Requirements[0].StructType == FTestFragment_Float::StaticStruct());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ExplicitRequirements, "System.Mass.Query.ExplicitRequiements");


struct FQueryTest_FragmentViewBinding : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);
		FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		AITEST_TRUE("Initial value of the fragment should match expectations", TestedFragment.Value == 0.f);

		UMassTestProcessor_Floats* Processor = NewTestProcessor<UMassTestProcessor_Floats>(EntityManager);
		Processor->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context) 
		{
			TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();
			for (int32 i = 0; i < Context.GetNumEntities(); ++i)
			{
				Floats[i].Value = 13.f;
			}
		};

		FMassProcessingContext ProcessingContext(*EntityManager, /*DeltaSeconds=*/0.f);
		UE::Mass::Executor::Run(*Processor, ProcessingContext);

		AITEST_EQUAL("Fragment value should have changed to the expected value", TestedFragment.Value, 13.f);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_FragmentViewBinding, "System.Mass.Query.FragmentViewBinding");

struct FQueryTest_ExecuteSingleArchetype : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 NumToCreate = 10;
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumToCreate, EntitiesCreated);
		
		int TotalProcessed = 0;

		FMassExecutionContext ExecContext(*EntityManager.Get());
		FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct() });
		Query.ForEachEntityChunk(ExecContext, [&TotalProcessed](FMassExecutionContext& Context)
			{
				TotalProcessed += Context.GetNumEntities();
				TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();
				
				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					Floats[i].Value = 13.f;
				}
			});

		AITEST_TRUE("The number of entities processed needs to match expectations", TotalProcessed == NumToCreate);

		for (FMassEntityHandle& Entity : EntitiesCreated)
		{
			const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			AITEST_EQUAL("Every fragment value should have changed to the expected value", TestedFragment.Value, 13.f);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ExecuteSingleArchetype, "System.Mass.Query.ExecuteSingleArchetype");


struct FQueryTest_ExecuteMultipleArchetypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 FloatsArchetypeCreated = 7;
		const int32 IntsArchetypeCreated = 11;
		const int32 FloatsIntsArchetypeCreated = 13;
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(IntsArchetype, IntsArchetypeCreated, EntitiesCreated);
		// clear to store only the float-related entities
		EntitiesCreated.Reset();
		EntityManager->BatchCreateEntities(FloatsArchetype, FloatsArchetypeCreated, EntitiesCreated);
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, FloatsIntsArchetypeCreated, EntitiesCreated);

		int TotalProcessed = 0;
		FMassExecutionContext ExecContext(*EntityManager.Get());
		FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct() });
		Query.ForEachEntityChunk(ExecContext, [&TotalProcessed](FMassExecutionContext& Context)
			{
				TotalProcessed += Context.GetNumEntities();
				TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					Floats[i].Value = 13.f;
				}
			});

		AITEST_TRUE("The number of entities processed needs to match expectations", TotalProcessed == FloatsIntsArchetypeCreated + FloatsArchetypeCreated);

		for (FMassEntityHandle& Entity : EntitiesCreated)
		{
			const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			AITEST_EQUAL("Every fragment value should have changed to the expected value", TestedFragment.Value, 13.f);
		}

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ExecuteMultipleArchetypes, "System.Mass.Query.ExecuteMultipleArchetypes");


struct FQueryTest_ExecuteSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 NumToCreate = 10;
		TArray<FMassEntityHandle> AllEntitiesCreated;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumToCreate, AllEntitiesCreated);
		
		TArray<int32> IndicesToProcess = { 1, 2, 3, 6, 7};
		TArray<FMassEntityHandle> EntitiesToProcess;
		TArray<FMassEntityHandle> EntitiesToIgnore;
		for (int32 i = 0; i < AllEntitiesCreated.Num(); ++i)
		{
			if (IndicesToProcess.Find(i) != INDEX_NONE)
			{
				EntitiesToProcess.Add(AllEntitiesCreated[i]);
			}
			else
			{
				EntitiesToIgnore.Add(AllEntitiesCreated[i]);
			}
		}


		int TotalProcessed = 0;

		FMassExecutionContext ExecContext(*EntityManager.Get());
		FMassEntityQuery TestQuery(EntityManager);
		TestQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
		TestQuery.ForEachEntityChunk(FMassArchetypeEntityCollection(FloatsArchetype, EntitiesToProcess, FMassArchetypeEntityCollection::NoDuplicates)
								, ExecContext, [&TotalProcessed](FMassExecutionContext& Context)
			{
				TotalProcessed += Context.GetNumEntities();
				TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					Floats[i].Value = 13.f;
				}
			});

		AITEST_TRUE("The number of entities processed needs to match expectations", TotalProcessed == IndicesToProcess.Num());

		for (FMassEntityHandle& Entity : EntitiesToProcess)
		{
			const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			AITEST_EQUAL("Every fragment value should have changed to the expected value", TestedFragment.Value, 13.f);
		}
		
		for (FMassEntityHandle& Entity : EntitiesToIgnore)
		{
			const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			AITEST_EQUAL("Untouched entities should retain default fragment value ", TestedFragment.Value, 0.f);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ExecuteSparse, "System.Mass.Query.ExecuteSparse");


struct FQueryTest_TagPresent : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		TArray<const UScriptStruct*> Fragments = {FTestFragment_Float::StaticStruct(), FTestFragment_Tag::StaticStruct()};
		const FMassArchetypeHandle FloatsTagArchetype = EntityManager->CreateArchetype(Fragments);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
		Query.AddTagRequirement<FTestFragment_Tag>(EMassFragmentPresence::All);
		Query.CacheArchetypes();

		AITEST_EQUAL("There's a single archetype matching the requirements", Query.GetArchetypes().Num(), 1);
		AITEST_TRUE("The only valid archetype is FloatsTagArchetype", FloatsTagArchetype == Query.GetArchetypes()[0]);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_TagPresent, "System.Mass.Query.TagPresent");


struct FQueryTest_TagAbsent : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		TArray<const UScriptStruct*> Fragments = { FTestFragment_Float::StaticStruct(), FTestFragment_Tag::StaticStruct() };
		const FMassArchetypeHandle FloatsTagArchetype = EntityManager->CreateArchetype(Fragments);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
		Query.AddTagRequirement<FTestFragment_Tag>(EMassFragmentPresence::None);
		Query.CacheArchetypes();

		AITEST_EQUAL("There are exactly two archetypes matching the requirements", Query.GetArchetypes().Num(), 2);
		AITEST_TRUE("FloatsTagArchetype is not amongst matching archetypes"
			, !(FloatsTagArchetype == Query.GetArchetypes()[0] || FloatsTagArchetype == Query.GetArchetypes()[1]));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_TagAbsent, "System.Mass.Query.TagAbsent");


/** using a fragment as a tag */
struct FQueryTest_FragmentPresent : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		// using EMassFragmentAccess::None to indicate we're interested only in the archetype having the fragment, no binding is required
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::Any);
		Query.CacheArchetypes();

		AITEST_EQUAL("There are exactly two archetypes matching the requirements", Query.GetArchetypes().Num(), 2);
		AITEST_TRUE("FloatsArchetype is not amongst matching archetypes"
			, !(FloatsArchetype == Query.GetArchetypes()[0] || FloatsArchetype == Query.GetArchetypes()[1]));

		constexpr int32 NumberOfEntitiesToAddA = 5;
		constexpr int32 NumberOfEntitiesToAddB = 7;
		TArray<FMassEntityHandle> MatchingEntities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumberOfEntitiesToAddA, MatchingEntities);
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumberOfEntitiesToAddB, MatchingEntities);
		ensure(MatchingEntities.Num() == NumberOfEntitiesToAddA + NumberOfEntitiesToAddB);

		int TotalProcessed = 0;
		FMassExecutionContext ExecContext(*EntityManager.Get());
		Query.ForEachEntityChunk(ExecContext, [&TotalProcessed](FMassExecutionContext& Context) {
			TotalProcessed += Context.GetNumEntities();
		});
		AITEST_EQUAL("We expect the number of entities processed to match number added to matching archetypes", MatchingEntities.Num(), TotalProcessed);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_FragmentPresent, "System.Mass.Query.FragmentPresent");


struct FQueryTest_OnlyAbsentFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		AITEST_FALSE("The empty query is not valid", Query.CheckValidity());

		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
		AITEST_TRUE("Single negative requirement is valid", Query.CheckValidity());

		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
		Query.AddRequirement<FTestFragment_Large>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::None);
		AITEST_TRUE("Multiple negative requirement is valid", Query.CheckValidity());
		
		Query.CacheArchetypes();
		AITEST_EQUAL("There's only one default test archetype matching the query", Query.GetArchetypes().Num(), 1);
		AITEST_TRUE("Only the Empty archetype matches the query", Query.GetArchetypes()[0] == EmptyArchetype);

		const FMassArchetypeHandle NewMatchingArchetypeHandle = EntityManager->CreateArchetype({ FTestFragment_Bool::StaticStruct() });
		Query.CacheArchetypes();
		AITEST_EQUAL("The number of matching queries matches expectations", Query.GetArchetypes().Num(), 2);
		AITEST_TRUE("The new archetype matches the query", Query.GetArchetypes()[1] == NewMatchingArchetypeHandle);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_OnlyAbsentFragments, "System.Mass.Query.OnlyAbsentFragments");


struct FQueryTest_AbsentAndPresentFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

		AITEST_TRUE("The query is valid", Query.CheckValidity());
		Query.CacheArchetypes();
		AITEST_EQUAL("There is only one archetype matching the query", Query.GetArchetypes().Num(), 1);
		AITEST_TRUE("FloatsArchetype is the only one matching the query", FloatsArchetype == Query.GetArchetypes()[0]);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_AbsentAndPresentFragments, "System.Mass.Query.AbsentAndPresentFragments");


struct FQueryTest_SingleOptionalFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Query.CacheArchetypes();

		AITEST_EQUAL("There are exactly two archetypes matching the requirements", Query.GetArchetypes().Num(), 2);
		AITEST_TRUE("FloatsArchetype is not amongst matching archetypes"
			, !(FloatsArchetype == Query.GetArchetypes()[0] || FloatsArchetype == Query.GetArchetypes()[1]));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_SingleOptionalFragment, "System.Mass.Query.SingleOptionalFragment");


struct FQueryTest_MultipleOptionalFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Query.CacheArchetypes();

		AITEST_EQUAL("All three archetype meet requirements", Query.GetArchetypes().Num(), 3);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_MultipleOptionalFragment, "System.Mass.Query.MultipleOptionalFragment");


/** This test configures a query to fetch archetypes that have a Float fragment (we have two of these) with an optional 
 *  Int fragment (of which we'll have one among the Float ones) */
struct FQueryTest_UsingOptionalFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		EntityManager->CreateEntity(FloatsArchetype);
		const FMassEntityHandle EntityWithFloatsInts = EntityManager->CreateEntity(FloatsIntsArchetype);
		EntityManager->CreateEntity(IntsArchetype);

		const int32 IntValueSet = 123;
		int TotalProcessed = 0;
		int EmptyIntsViewCount = 0;

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
		FMassExecutionContext ExecContext(*EntityManager.Get());
		Query.ForEachEntityChunk(ExecContext, [&TotalProcessed, &EmptyIntsViewCount, IntValueSet](FMassExecutionContext& Context) {
			++TotalProcessed;
			TArrayView<FTestFragment_Int> Ints = Context.GetMutableFragmentView<FTestFragment_Int>();
			if (Ints.Num() == 0)
			{
				++EmptyIntsViewCount;
			}
			else
			{
				for (FTestFragment_Int& IntFragment : Ints)
				{	
					IntFragment.Value = IntValueSet;
				}
			}
			});

		AITEST_EQUAL("Two archetypes total should get processed", TotalProcessed, 2);
		AITEST_EQUAL("Only one of these archetypes should get an empty Ints array view", EmptyIntsViewCount, 1);

		const FTestFragment_Int& TestFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(EntityWithFloatsInts);
		AITEST_TRUE("The optional fragment\'s value should get modified where present", TestFragment.Value == IntValueSet);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_UsingOptionalFragment, "System.Mass.Query.UsingOptionalFragment");


struct FQueryTest_AnyFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// From FEntityTestBase:
		// FMassArchetypeHandle FloatsArchetype;
		// FMassArchetypeHandle IntsArchetype;
		// FMassArchetypeHandle FloatsIntsArchetype;
		const FMassArchetypeHandle BoolArchetype = EntityManager->CreateArchetype({ FTestFragment_Bool::StaticStruct() });
		const FMassArchetypeHandle BoolFloatArchetype = EntityManager->CreateArchetype({ FTestFragment_Bool::StaticStruct(), FTestFragment_Float::StaticStruct() });

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);
		Query.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);
		// this query should match: 
		// IntsArchetype, FloatsIntsArchetype, BoolArchetype, BoolFloatArchetype
		Query.CacheArchetypes();

		AITEST_EQUAL("Archetypes containing Int or Bool should meet requirements", Query.GetArchetypes().Num(), 4);

		// populate the archetypes so that we can test fragment binding
		for (auto ArchetypeHandle : Query.GetArchetypes())
		{
			EntityManager->CreateEntity(ArchetypeHandle);
		}

		FMassExecutionContext TestContext(*EntityManager.Get());
		Query.ForEachEntityChunk(TestContext, [this](FMassExecutionContext& Context)
			{
				TArrayView<FTestFragment_Bool> BoolView = Context.GetMutableFragmentView<FTestFragment_Bool>();
				TArrayView<FTestFragment_Int> IntView = Context.GetMutableFragmentView<FTestFragment_Int>();
				
				GetTestRunner().TestTrue("Every matching archetype needs to host Bool or Int fragments", BoolView.Num() || IntView.Num());
			});

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_AnyFragment, "System.Mass.Query.AnyFragment");


struct FQueryTest_AnyTag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const FMassArchetypeHandle ABArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct(), FTestTag_B::StaticStruct() });
		const FMassArchetypeHandle ACArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_A::StaticStruct(), FTestTag_C::StaticStruct() });
		const FMassArchetypeHandle BCArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_B::StaticStruct(), FTestTag_C::StaticStruct() });
		const FMassArchetypeHandle BDArchetype = EntityManager->CreateArchetype({ FTestFragment_Int::StaticStruct(), FTestTag_B::StaticStruct(), FTestTag_D::StaticStruct() });
		const FMassArchetypeHandle FloatACArchetype = EntityManager->CreateArchetype({ FTestFragment_Float::StaticStruct(), FTestTag_A::StaticStruct(), FTestTag_C::StaticStruct() });

		FMassEntityQuery Query(EntityManager);
		// at least one fragment requirement needs to be present for the query to be valid
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly); 
		Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Any);
		Query.AddTagRequirement<FTestTag_C>(EMassFragmentPresence::Any);
		// this query should match: 
		// ABArchetype, ACArchetype and ABCrchetype but not BDArchetype nor FEntityTestBase.IntsArchetype
		Query.CacheArchetypes();

		AITEST_EQUAL("Only Archetypes tagged with A or C should matched the query", Query.GetArchetypes().Num(), 3);
		AITEST_TRUE("ABArchetype should be amongst the matched archetypes", Query.GetArchetypes().Find(ABArchetype) != INDEX_NONE);
		AITEST_TRUE("ACArchetype should be amongst the matched archetypes", Query.GetArchetypes().Find(ACArchetype) != INDEX_NONE);
		AITEST_TRUE("BCArchetype should be amongst the matched archetypes", Query.GetArchetypes().Find(BCArchetype) != INDEX_NONE);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_AnyTag, "System.Mass.Query.AnyTag");


struct FQueryTest_AutoRecache : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		// at least one fragment requirement needs to be present for the query to be valid
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		
		int32 EntitiesFound = 0;
		const FMassExecuteFunction QueryExecFunction = [&EntitiesFound](FMassExecutionContext& Context)
		{
			EntitiesFound += Context.GetNumEntities();
		};
		
		FMassExecutionContext ExecutionContext(*EntityManager, /*DeltaSeconds=*/0.f);
		Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);
		
		AITEST_EQUAL("No entities have been created so we expect counting to yield 0", EntitiesFound, 0);

		constexpr int32 NumberOfEntitiesMatching = 17;
		TArray<FMassEntityHandle> MatchingEntities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumberOfEntitiesMatching, MatchingEntities);

		EntitiesFound = 0;
		Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);
		AITEST_EQUAL("The number of entities found should match the number of entities created in the matching archetype", EntitiesFound, MatchingEntities.Num());

		// create more entities, but in an archetype not matching the query
		constexpr int32 NumberOfEntitiesNotMatching = 13;
		TArray<FMassEntityHandle> NotMatchingEntities;
		EntityManager->BatchCreateEntities(FloatsArchetype, NumberOfEntitiesNotMatching, NotMatchingEntities);
		EntitiesFound = 0;
		Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);
		AITEST_EQUAL("The number of entities found should not change with addition of entities not matching the query", EntitiesFound, MatchingEntities.Num());

		// create some more in another matching archetype
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, NumberOfEntitiesMatching, MatchingEntities);
		EntitiesFound = 0;
		Query.ForEachEntityChunk(ExecutionContext, QueryExecFunction);
		AITEST_EQUAL("The total number of entities found should include entities from both matching archetypes", EntitiesFound, MatchingEntities.Num());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_AutoRecache, "System.Mass.Query.AutoReCaching");


struct FQueryTest_AllOptional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::None, EMassFragmentPresence::Optional);
		Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Optional);
		Query.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::Optional);
		Query.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::Optional);
		Query.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::Optional);

		Query.CacheArchetypes();

		int32 ExpectedNumOfArchetypes = 2;
		// only the FloatsArchetype and FloatsIntsArchetype should match
		AITEST_TRUE("Initial number of matching archetypes matches expectations", Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);

		int32 CurrentEntityIndex = 0;

		EntityManager->AddTagToEntity(Entities[CurrentEntityIndex++], FTestTag_A::StaticStruct());
		++ExpectedNumOfArchetypes;
		EntityManager->AddTagToEntity(Entities[CurrentEntityIndex++], FTestTag_B::StaticStruct());
		Query.CacheArchetypes();
		AITEST_EQUAL("A: number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);

		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestChunkFragment_Int>();
			EntityManager->CreateArchetype(Descriptor);
			++ExpectedNumOfArchetypes;
		}
		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestChunkFragment_Float>();
			EntityManager->CreateArchetype(Descriptor);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("B: number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);

		{
			FTestSharedFragment_Int FragmentInstance;
			FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
			FMassArchetypeSharedFragmentValues SharedFragmentValues;
			SharedFragmentValues.Add(SharedFragmentInstance);

			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
			EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
			++ExpectedNumOfArchetypes;
		}
		{
			FTestSharedFragment_Float FragmentInstance;
			FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
			FMassArchetypeSharedFragmentValues SharedFragmentValues;
			SharedFragmentValues.Add(SharedFragmentInstance);

			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
			EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("C: number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);

		{
			FTestConstSharedFragment_Int FragmentInstance;
			FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
			EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
			++ExpectedNumOfArchetypes;
		}
		{
			FTestConstSharedFragment_Float FragmentInstance;
			FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
			EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("D: number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_AllOptional, "System.Mass.Query.AllOptional");

struct FQueryTest_JustATag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassEntityQuery Query(EntityManager);
		Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		Query.CacheArchetypes();

		int32 ExpectedNumOfArchetypes = 0;
		// only the FloatsArchetype and FloatsIntsArchetype should match
		AITEST_TRUE("Initial number of matching archetypes matches expectations", Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestTag_A>();
			EntityManager->CreateArchetype(Descriptor);
			++ExpectedNumOfArchetypes;
		}
		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestTag_B>();
			EntityManager->CreateArchetype(Descriptor);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("A: number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);

		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestTag_A>();
			Descriptor.Add<FTestTag_C>();
			Descriptor.Add<FTestTag_D>();
			EntityManager->CreateArchetype(Descriptor);
			++ExpectedNumOfArchetypes;
		}
		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestTag_B>();
			Descriptor.Add<FTestTag_C>();
			Descriptor.Add<FTestTag_D>();
			EntityManager->CreateArchetype(Descriptor);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("B: number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_JustATag, "System.Mass.Query.JustATag");

struct FQueryTest_JustAChunkFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassArchetypeHandle TargetArchetype;

		FMassEntityQuery Query(EntityManager);
		Query.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Query.CacheArchetypes();

		int32 ExpectedNumOfArchetypes = 0;
		// no matching archetypes at this time
		AITEST_TRUE("Initial number of matching archetypes matches expectations", Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestChunkFragment_Int>();
			TargetArchetype = EntityManager->CreateArchetype(Descriptor);
			++ExpectedNumOfArchetypes;
		}
		{
			FMassArchetypeCompositionDescriptor Descriptor(EntityManager->GetArchetypeComposition(IntsArchetype));
			Descriptor.Add<FTestChunkFragment_Float>();
			EntityManager->CreateArchetype(Descriptor);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("Number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);
		
		// try to access the chunk fragment
		{
			EntityManager->CreateEntity(TargetArchetype);

			FMassExecutionContext ExecContext(*EntityManager.Get());
			bool bExecuted = false;
			Query.ForEachEntityChunk(ExecContext, [&bExecuted](FMassExecutionContext& Context)
				{
					const FTestChunkFragment_Int& ChunkFragment = Context.GetChunkFragment<FTestChunkFragment_Int>();
					bExecuted = true;
				});
			AITEST_TRUE("The tested query did execute and bounding was successful", bExecuted);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_JustAChunkFragment, "System.Mass.Query.JustAChunkFragment");


struct FQueryTest_JustASharedFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassArchetypeHandle TargetArchetype;

		FMassEntityQuery Query(EntityManager);
		Query.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Query.CacheArchetypes();

		int32 ExpectedNumOfArchetypes = 0;
		// no matching archetypes at this time
		AITEST_TRUE("Initial number of matching archetypes matches expectations", Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);
		int32 CurrentEntityIndex = 0;

		{
			FTestSharedFragment_Int FragmentInstance;
			FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
			FMassArchetypeSharedFragmentValues SharedFragmentValues;
			SharedFragmentValues.Add(SharedFragmentInstance);

			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
			EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
			++ExpectedNumOfArchetypes;
		}
		{
			FTestSharedFragment_Float FragmentInstance;
			FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
			FMassArchetypeSharedFragmentValues SharedFragmentValues;
			SharedFragmentValues.Add(SharedFragmentInstance);

			FMassArchetypeEntityCollection Collection(IntsArchetype, MakeArrayView(&Entities[CurrentEntityIndex++], 1), FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
			EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&Collection, 1), SharedFragmentValues);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("Number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);

		// try to access the shared fragment
		{
			bool bExecuted = false;
			FMassExecutionContext ExecContext(*EntityManager.Get());
			Query.ForEachEntityChunk(ExecContext, [&bExecuted](FMassExecutionContext& Context)
				{
					const FTestSharedFragment_Int& SharedFragment = Context.GetSharedFragment<FTestSharedFragment_Int>();
					bExecuted = true;
				});
			AITEST_TRUE("The tested query did execute and bounding was successful", bExecuted);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_JustASharedFragment, "System.Mass.Query.JustASharedFragment");


struct FQueryTest_JustAConstSharedFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassArchetypeHandle TargetArchetype;

		FMassEntityQuery Query(EntityManager);
		Query.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::All);
		Query.CacheArchetypes();

		int32 ExpectedNumOfArchetypes = 0;
		// no matching archetypes at this time
		AITEST_TRUE("Initial number of matching archetypes matches expectations", Query.GetArchetypes().Num() == ExpectedNumOfArchetypes);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);
		int32 CurrentEntityIndex = 0;

		{
			FTestConstSharedFragment_Int FragmentInstance;
			FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
			EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
			++ExpectedNumOfArchetypes;
		}
		{
			FTestConstSharedFragment_Float FragmentInstance;
			FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
			EntityManager->AddConstSharedFragmentToEntity(Entities[CurrentEntityIndex++], SharedFragmentInstance);
		}
		Query.CacheArchetypes();
		AITEST_EQUAL("Number of matching archetypes matches expectations.", Query.GetArchetypes().Num(), ExpectedNumOfArchetypes);

		// try to access the shared fragment
		{
			bool bExecuted = false;
			FMassExecutionContext ExecContext(*EntityManager.Get());
			Query.ForEachEntityChunk(ExecContext, [&bExecuted](FMassExecutionContext& Context)
				{
					const FTestConstSharedFragment_Int& SharedFragment = Context.GetConstSharedFragment<FTestConstSharedFragment_Int>();
					bExecuted = true;
				});
			AITEST_TRUE("The tested query did execute and bounding was successful", bExecuted);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_JustAConstSharedFragment, "System.Mass.Query.JustAConstSharedFragment");


struct FQueryTest_GameThreadOnly : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		EntityManager->GetTypeManager().RegisterType<FTestSharedFragment_Int>();
		EntityManager->GetTypeManager().RegisterType<UMassTestWorldSubsystem>();

		{
			FMassEntityQuery Query(EntityManager);
			Query.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
			AITEST_EQUAL("Statically typed shared fragment", Query.DoesRequireGameThreadExecution(), TMassSharedFragmentTraits<FTestSharedFragment_Int>::GameThreadOnly);
		}
		{
			FMassEntityQuery Query(EntityManager);
			Query.AddSharedRequirement(FTestSharedFragment_Int::StaticStruct(), EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);
			AITEST_EQUAL("Statically typed shared fragment", Query.DoesRequireGameThreadExecution(), TMassSharedFragmentTraits<FTestSharedFragment_Int>::GameThreadOnly);
		}
		{
			FMassEntityQuery Query(EntityManager);
			Query.AddSubsystemRequirement<UMassTestWorldSubsystem>(EMassFragmentAccess::ReadWrite);
			AITEST_EQUAL("Statically typed shared fragment", Query.DoesRequireGameThreadExecution(), TMassSharedFragmentTraits<UMassTestWorldSubsystem>::GameThreadOnly);
		}
		{
			FMassEntityQuery Query(EntityManager);
			Query.AddSubsystemRequirement(UMassTestWorldSubsystem::StaticClass(), EMassFragmentAccess::ReadWrite);
			AITEST_EQUAL("Statically typed shared fragment", Query.DoesRequireGameThreadExecution(), TMassSharedFragmentTraits<UMassTestWorldSubsystem>::GameThreadOnly);
		}
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_GameThreadOnly, "System.Mass.Query.GameThreadOnly");

struct FQueryTest_ExportHandles : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 EntitiesPerChunk = 16384;
		constexpr int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);
		check(Entities.Num() == 2*Count);

		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager.Get(), Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);
		EntityManager->BatchChangeTagsForEntities(EntityCollections, FMassTagBitSet(FTestTag_A::StaticStruct()), FMassTagBitSet());

		FMassEntityQuery Query(EntityManager.ToSharedRef());
		Query.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);

		TArray<FMassEntityHandle> QueryMatchingEntities = Query.GetMatchingEntityHandles();

		Entities.Sort();
		QueryMatchingEntities.Sort();
		AITEST_TRUE("Exported handle list contain all the expected handles", Algo::Compare(Entities, QueryMatchingEntities));
		
		TArray<FMassArchetypeEntityCollection> MatchingCollections = Query.CreateMatchingEntitiesCollection();
		AITEST_EQUAL("Expected number of archetypes in resulting collections", MatchingCollections.Num(), 2);

		TArray<FMassEntityHandle> HandlesFromCollections;
		for (const FMassArchetypeEntityCollection& Collection : MatchingCollections)
		{
			Collection.ExportEntityHandles(HandlesFromCollections);
		}
		HandlesFromCollections.Sort();
		AITEST_TRUE("Handles exported from the collections contain all the expected handles", Algo::Compare(Entities, HandlesFromCollections));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ExportHandles, "System.Mass.Query.ExportHandles");

struct FQueryTest_ExecutionLimiter : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 EntityLimit = 1;
		const int32 CyclesToSaturate = 20; // no assumptions about chunk size, the minimum cycles will be the total entity count needing processing
		const int32 FloatsArchetypeCreated = 7;
		const int32 IntsArchetypeCreated = 11;
		const int32 FloatsIntsArchetypeCreated = 13;
		TArray<FMassEntityHandle> EntitiesCreated;
		EntityManager->BatchCreateEntities(IntsArchetype, IntsArchetypeCreated, EntitiesCreated);
		// clear to store only the float-related entities
		EntitiesCreated.Reset();
		EntityManager->BatchCreateEntities(FloatsArchetype, FloatsArchetypeCreated, EntitiesCreated);
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, FloatsIntsArchetypeCreated, EntitiesCreated);

		int TotalProcessed = 0;
		FMassExecutionContext ExecContext(*EntityManager.Get());
		FMassEntityQuery Query(EntityManager.ToSharedRef(), { FTestFragment_Float::StaticStruct() });
		// limit the execution to a single entity per cycle (this will process the entire chunk, but only one no matter how many entities it contains)
		UE::Mass::FExecutionLimiter Limiter(EntityLimit);
		Query.ForEachEntityChunk(ExecContext, Limiter, [&TotalProcessed](FMassExecutionContext& Context)
			{
				TotalProcessed += Context.GetNumEntities();
				TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					Floats[i].Value = 13.f;
				}
			});

		AITEST_TRUE("Limiter should prevent all entities from being processed", TotalProcessed < FloatsIntsArchetypeCreated + FloatsArchetypeCreated);
		AITEST_TRUE("Limiter should process at least as many entities as requested", TotalProcessed >= EntityLimit);

		// run a few iterations to ensure all entities are processed:
		for (int i = 0; i < CyclesToSaturate; i++)
		{
			Query.ForEachEntityChunk(ExecContext, Limiter, [&TotalProcessed](FMassExecutionContext& Context)
				{
					TotalProcessed += Context.GetNumEntities();
					TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

					for (int32 i = 0; i < Context.GetNumEntities(); ++i)
					{
						Floats[i].Value = 13.f;
					}
				});
		}

		for (FMassEntityHandle& Entity : EntitiesCreated)
		{
			const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			AITEST_EQUAL("Every fragment value should have changed to the expected value", TestedFragment.Value, 13.f);
		}

		Limiter.EntityLimit = 1000000;
		TotalProcessed = 0;
		Query.ForEachEntityChunk(ExecContext, Limiter, [&TotalProcessed](FMassExecutionContext& Context)
			{
				TotalProcessed += Context.GetNumEntities();
				TArrayView<FTestFragment_Float> Floats = Context.GetMutableFragmentView<FTestFragment_Float>();

				for (int32 i = 0; i < Context.GetNumEntities(); ++i)
				{
					Floats[i].Value = 13.f;
				}
			});

		AITEST_TRUE("All entities should be processed with a high enough limit", TotalProcessed == FloatsIntsArchetypeCreated + FloatsArchetypeCreated);

		for (FMassEntityHandle& Entity : EntitiesCreated)
		{
			const FTestFragment_Float& TestedFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			AITEST_EQUAL("Every fragment value should have changed to the expected value", TestedFragment.Value, 13.f);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_ExecutionLimiter, "System.Mass.Query.ExecutionLimiter");


struct FQueryTest_LinkedFragmentAccess : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const int32 DirectIntValue = 1234;
		const int32 LinkedIntValue = 9898;
		const float DirectFloatValue = 10.0f;
		const float InirectFloatValue = -10.0f;

		TArray<const UScriptStruct*> Fragments = { FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct() };
		const FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(Fragments);
		FMassEntityHandle LinkedEntity = EntityManager->CreateEntity(Archetype);
		FTestFragment_Float& LinkedFloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(LinkedEntity);
		FTestFragment_Int& LinkedIntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(LinkedEntity);
		LinkedFloatFragment.Value = InirectFloatValue;
		LinkedIntFragment.Value = LinkedIntValue;

		FMassEntityLinkFragment LinkedEntityFragment;
		LinkedEntityFragment.LinkedEntityHandle = LinkedEntity;
		FConstSharedStruct SharedLinkedEntityFragment = EntityManager->GetOrCreateConstSharedFragment(LinkedEntityFragment);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, 10, Entities);
		for (FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, SharedLinkedEntityFragment);
			FTestFragment_Int& DirectIntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
			DirectIntFragment.Value = DirectIntValue;
		}

		Entities.Reset();
		EntityManager->BatchCreateEntities(FloatsArchetype, 10, Entities);
		for (FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddConstSharedFragmentToEntity(Entity, SharedLinkedEntityFragment);
			FTestFragment_Float& DirectFloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			DirectFloatFragment.Value = DirectFloatValue;
		}

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.AddLinkedEntityRequirement<FTestFragment_Int, FMassEntityLinkFragment>(EMassFragmentAccess::ReadOnly);
		Query.CacheArchetypes();

		bool bAnyMatch = false;
		bool bFoundShared = true;
		FMassExecutionContext ExecContext(*EntityManager.Get());
		Query.ForEachEntityChunk(ExecContext, [this, &bAnyMatch, &bFoundShared](FMassExecutionContext& Context)
			{
				const FTestFragment_Int* SharedInt = Context.GetLinkedEntityConstFragmentPtr<FTestFragment_Int>();
				if (!SharedInt)
				{
					bFoundShared = false;
					return;
				}
				const TConstArrayView<FTestFragment_Int> IntFragments = Context.GetFragmentView<FTestFragment_Int>();

				for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					const FTestFragment_Int& IntFragment = IntFragments[EntityIt];
					bAnyMatch |= IntFragment.Value == SharedInt->Value;
				}
			});
		AITEST_TRUE("linked fragment should be discoverable", bFoundShared);
		AITEST_FALSE("owned fragment should be independent from linked fragment", bAnyMatch);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_LinkedFragmentAccess, "System.Mass.Query.LinkedFragmentAccess");

struct FQueryTest_IndirectFragmentAccess : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);
		const int32 IntValue = 1234;
		const float FloatValue = 10.0f;
		const int32 EntityCount = 100;

		TArray<FMassEntityHandle> IntEntities;
		EntityManager->BatchCreateEntities(IntsArchetype, EntityCount, IntEntities);
		for (FMassEntityHandle& Entity : IntEntities)
		{
			FTestFragment_Int& IntFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
			IntFragment.Value = IntValue;
		}

		TArray<FMassEntityHandle> FloatEntities;
		EntityManager->BatchCreateEntities(FloatsArchetype, EntityCount, FloatEntities);
		int HandleIndex = 0;
		for (FMassEntityHandle& Entity : FloatEntities)
		{
			EntityManager->AddFragmentToEntity(Entity, FTestFragment_EntityHandle::StaticStruct());
			FTestFragment_Float& FloatFragment = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
			FTestFragment_EntityHandle& EntityhandleFragment = EntityManager->GetFragmentDataChecked<FTestFragment_EntityHandle>(Entity);
			FloatFragment.Value = FloatValue;
			EntityhandleFragment.Value = IntEntities[HandleIndex++];
		}

		FMassEntityQuery Query(EntityManager);
		Query.AddRequirement<FTestFragment_EntityHandle>(EMassFragmentAccess::ReadOnly);
		Query.AddIndirectFragmentRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
		Query.CacheArchetypes();

		bool bAllMatch = true;
		bool bFoundIndirect = true;
		FMassExecutionContext ExecContext(*EntityManager.Get());
		Query.ForEachEntityChunk(ExecContext, [this, &bAllMatch, &bFoundIndirect](FMassExecutionContext& Context)
			{
				const TConstArrayView<FTestFragment_EntityHandle> HandleFragments = Context.GetFragmentView<FTestFragment_EntityHandle>();

				for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
				{
					const FTestFragment_EntityHandle& HandleFragment = HandleFragments[EntityIt];
					const FTestFragment_Int* IntFragmentPtr = Context.GetIndirectConstFragmentPtr<FTestFragment_Int>(HandleFragment.Value);
					if (IntFragmentPtr)
					{
						bAllMatch |= IntFragmentPtr->Value == IntValue;
					}
					else
					{
						bFoundIndirect = false;
					}
				}
			});
		AITEST_TRUE("Indirect fragment should be discoverable", bFoundIndirect);
		AITEST_TRUE("Indirect fragment should have correct value", bAllMatch);
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FQueryTest_IndirectFragmentAccess, "System.Mass.Query.IndirectFragmentAccess");

} // FMassQueryTest

#undef LOCTEXT_NAMESPACE

