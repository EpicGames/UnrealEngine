// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityBuilder.h"
#include "MassRelationManager.h"
#include "MassEntityTestTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Relations
{
	using namespace UE::Mass;

	struct FRelationTestBase : FEntityTestBase
	{
		UScriptStruct* MyRelationType = nullptr;
		const FRelationTypeTraits* RelationTraits = nullptr;
		FTypeHandle RelationTypeHandle;
		FRelationManager* RelationshipManagerPtr = nullptr;

		static TNotNull<UScriptStruct*> MakeRelationType(FName TypeName)
		{
			UScriptStruct* NewStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(), FName(*FString::Printf(TEXT("Relation_%s"), *TypeName.ToString())), RF_Public);
			NewStruct->SetSuperStruct(FMassRelation::StaticStruct());

			return NewStruct;
		}

		template<UE::Mass::CElement TBase>
		static TNotNull<UScriptStruct*> MakeElementType(FName TypeName)
		{	
			UScriptStruct* NewStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(), FName(*FString::Printf(TEXT("%s"), *TypeName.ToString())), RF_Public);
			NewStruct->SetSuperStruct(TBase::StaticStruct());
			return NewStruct;
		}

		static FRelationTypeTraits CreateRelationTypeDescription(const FName FragmentName)
		{
			UScriptStruct* FragmentType = MakeRelationType(FragmentName);
			return CreateRelationTypeDescription(FragmentType);
		}

		static FRelationTypeTraits CreateRelationTypeDescription(TNotNull<UScriptStruct*> RelationType)
		{
			FRelationTypeTraits RelationTraits(RelationType);
			RelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::Destroy;
			RelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Object)].bExclusive = true;
			RelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::CleanUp;
			RelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].bExclusive = false;
			RelationTraits.bHierarchical = true;
			return RelationTraits;
		}

		virtual bool SetUp() override
		{
			if (FEntityTestBase::SetUp())
			{
				MyRelationType = MakeRelationType("MyChildOfRelation");
				FRelationTypeTraits TempRelationTraits = CreateRelationTypeDescription("MyChildOfRelation");
				TweakRelation(TempRelationTraits);
				RelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(TempRelationTraits));
				RelationTraits = &EntityManager->GetTypeManager().GetRelationTypeChecked(RelationTypeHandle);
				RelationshipManagerPtr = &EntityManager->GetRelationManager();
			}
			return RelationTypeHandle.IsValid();
		}

		virtual void TweakRelation(FRelationTypeTraits& TempRelationTraits)
		{
		}

		FMassEntityHandle GetRelationSubject(const FMassEntityHandle ObjectEntity) const 
		{
			check(RelationshipManagerPtr);
			TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ObjectEntity);
			return Subjects.Num() ? Subjects[0] : FMassEntityHandle();
		}

		FMassEntityHandle GetRelationObject(const FMassEntityHandle SubjectEntity) const
		{
			check(RelationshipManagerPtr);
			TArray<FMassEntityHandle> Objects = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, SubjectEntity);
			return Objects.Num() ? Objects[0] : FMassEntityHandle();
		}
	};

	struct FSingleRelationCreation : FRelationTestBase
	{
		FMassArchetypeHandle OriginalArchetype;
		TArray<FMassEntityHandle> CreatedEntities;

		virtual bool SetUp() override
		{
			FRelationTestBase::SetUp();
			OriginalArchetype = IntsArchetype;
			EntityManager->BatchCreateEntities(OriginalArchetype, 2, CreatedEntities);
			return true;
		}

		virtual bool InstantTest() override
		{
			const FMassEntityHandle ChildEntity = CreatedEntities[0];
			const FMassEntityHandle ParentEntity = CreatedEntities[1];

			FMassEntityHandle CreatedRelationEntity = RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, ChildEntity, ParentEntity);
			EntityManager->FlushCommands();

			AITEST_TRUE("Valid relation entity has been created", CreatedRelationEntity.IsValid());

			TArray<FMassEntityHandle> TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ChildEntity);
			AITEST_TRUE("No subjects point to the ChildEntity as the relation's object", TestedEntities.IsEmpty());
			TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ParentEntity);
			AITEST_TRUE("ParentEntity is an object of a relation of the given type", TestedEntities.Num() > 0);
			AITEST_TRUE("ParentEntity is an object of exactly one relation of the given type", TestedEntities.Num() == 1);
			AITEST_TRUE("The ChildEntity is the subject of the give relation, where ParentEntity is the object", TestedEntities[0] == ChildEntity);

			TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ChildEntity);
			AITEST_TRUE("The ChildEntity is has exactly one object for the given relation type", TestedEntities.Num() == 1);
			AITEST_TRUE("The object for the given relation for the ChildEntity is the ParentEntity", TestedEntities[0] == ParentEntity);
			TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ParentEntity);
			AITEST_TRUE("The ParentEntity has no objects for the given relation type", TestedEntities.IsEmpty());

			RelationshipManagerPtr->DestroyRelationInstance(RelationTypeHandle, ChildEntity, ParentEntity);
			EntityManager->FlushCommands();
			TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ChildEntity);
			AITEST_TRUE("No relation subjects for ChildEntity still", TestedEntities.IsEmpty());
			TestedEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, ParentEntity);
			AITEST_TRUE("No relation subjects for ParentEntity anymore", TestedEntities.IsEmpty());

			TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ChildEntity);
			AITEST_TRUE("No relation objects for ChildEntity anymore", TestedEntities.IsEmpty());
			TestedEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ParentEntity);
			AITEST_TRUE("No relation objects for ParentEntity still", TestedEntities.IsEmpty());

			return true;
		}

	};
	IMPLEMENT_AI_INSTANT_TEST(FSingleRelationCreation, "System.Mass.Relations.CreateSingle");


	struct FUnitHierarchy : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, 2, Entities);

			constexpr int32 ParentIndex = 1;
			constexpr int32 ChildIndex = 0;
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[ChildIndex], Entities[ParentIndex]);

			const FMassArchetypeHandle ArchetypeParent = EntityManager->GetArchetypeForEntity(Entities[ParentIndex]);
			const FMassArchetypeHandle ArchetypeChild = EntityManager->GetArchetypeForEntity(Entities[ChildIndex]);

			AITEST_NOT_EQUAL("Child and Parent archetypes", ArchetypeParent, ArchetypeChild);
			AITEST_NOT_EQUAL("Child and the original archetype", ArchetypeChild, OriginalArchetype);
			AITEST_NOT_EQUAL("Parent and the original archetype", ArchetypeParent, OriginalArchetype);

			FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[ChildIndex], RelationTraits->RegisteredGroupType);
			AITEST_EQUAL("Leaf entity's is at expected group level", GroupHandle.GetGroupID(), 1);
			
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FUnitHierarchy, "System.Mass.Relations.Hierarchy.Unit");

	struct FChainHierarchy : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 3;
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

			constexpr int32 LeafIndex = 1;
			constexpr int32 MiddleIndex = 0;
			constexpr int32 RootIndex = 2;
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[LeafIndex], Entities[MiddleIndex]);
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[MiddleIndex], Entities[RootIndex]);

			FMassArchetypeHandle Archetypes[NumEntities] = {
				EntityManager->GetArchetypeForEntity(Entities[0])
				, EntityManager->GetArchetypeForEntity(Entities[1])
				, EntityManager->GetArchetypeForEntity(Entities[2])
			};

			for (int32 Index = 0; Index < NumEntities; ++Index)
			{
				AITEST_NOT_EQUAL("Each pair of archetypes", Archetypes[Index], Archetypes[(Index + 1) % NumEntities]);
			}
			{
				constexpr int32 EntityIndex = LeafIndex;
				const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
				
				AITEST_EQUAL("Leaf entity's Object vs Middle", Object, Entities[MiddleIndex]);
				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[EntityIndex]);
				AITEST_TRUE("Leaf entity's has no children", Subjects.IsEmpty());

				FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[EntityIndex], RelationTraits->RegisteredGroupType);
				AITEST_EQUAL("Leaf entity's is at expected group level", GroupHandle.GetGroupID(), 2);
			}
			{
				constexpr int32 EntityIndex = MiddleIndex;
				const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
				AITEST_EQUAL("Middle entity's Object vs Root", Object, Entities[RootIndex]);
				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle,  Entities[EntityIndex]);
				AITEST_TRUE("Middle entity's has a single child", Subjects.Num() == 1);
				AITEST_TRUE("Middle entity's sole child is the leaf entity", Subjects.Last() == Entities[LeafIndex]);

				FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[EntityIndex], RelationTraits->RegisteredGroupType);
				AITEST_EQUAL("Leaf entity's is at expected group level", GroupHandle.GetGroupID(), 1);
			}
			{
				constexpr int32 EntityIndex = RootIndex;
				const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
				AITEST_TRUE("Root entity has no parent", Object.IsValid() == false);
				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle,  Entities[EntityIndex]);
				AITEST_TRUE("Root entity's has a single child", Subjects.Num() == 1);
				AITEST_TRUE("Root entity's sole child is the middle entity", Subjects.Last() == Entities[MiddleIndex]);

				FArchetypeGroupHandle GroupHandle = EntityManager->GetGroupForEntity(Entities[EntityIndex], RelationTraits->RegisteredGroupType);
				AITEST_EQUAL("Leaf entity's is at expected group level", GroupHandle.GetGroupID(), 0);
			}
			
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChainHierarchy, "System.Mass.Relations.Hierarchy.Chain");

	struct FHierarchyTrivialReparenting : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			if (FEntityBuilder::GetDefaultValueOfForceDeferredCommit())
			{
				// this test assumes synchronous EntityBuilder commits. Bailing out
				// to avoid generating false positives
				return true;
			}

			constexpr int32 NumEntities = 3;
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

			constexpr int32 LeafIndex = 1;
			constexpr int32 OriginalParentIndex = 0;
			constexpr int32 FinalParentIndex = 2;
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[LeafIndex], Entities[OriginalParentIndex]);
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[LeafIndex], Entities[FinalParentIndex]);
			{
				constexpr int32 EntityIndex = LeafIndex;
				const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
				AITEST_NOT_EQUAL("Leaf entity's Object vs Original Parent", Object, Entities[OriginalParentIndex]);
				AITEST_EQUAL("Leaf entity's Object vs Final Parent", Object, Entities[FinalParentIndex]);

				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle,  Entities[EntityIndex]);
				AITEST_TRUE("Leaf entity's has no children", Subjects.IsEmpty());
			}
			{
				constexpr int32 EntityIndex = OriginalParentIndex;
				const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
				AITEST_FALSE("Original Parent has a parent", Object.IsValid());

				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle,  Entities[EntityIndex]);
				AITEST_TRUE("Original Parent has no children", Subjects.IsEmpty());
			}
			{
				constexpr int32 EntityIndex = FinalParentIndex;
				const FMassEntityHandle Object = GetRelationObject(Entities[EntityIndex]);
				AITEST_FALSE("Final Parent has a parent", Object.IsValid());

				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle,  Entities[EntityIndex]);
				AITEST_TRUE("Final Parent has one child", Subjects.Num() == 1);
				AITEST_TRUE("Final Parent's sole child is the Leaf entity", Subjects.Last() == Entities[LeafIndex]);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FHierarchyTrivialReparenting, "System.Mass.Relations.Hierarchy.TrivialReparenting");

	struct FComplexHierarchy : FRelationTestBase
	{
		using Super = FRelationTestBase;

		int32 NoRelation = INDEX_NONE;
		int32 SingleChildParent = INDEX_NONE;
		int32 SingleChildChild = INDEX_NONE;
		static constexpr int32 NumEntities = 9;
		bool Result[NumEntities][NumEntities] = {
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
				{ false, false, false, false, false, false, false, false, false},
			};

		FMassArchetypeHandle OriginalArchetype;
		TArray<FMassEntityHandle> Entities;

		virtual bool SetUp() override
		{
			Super::SetUp();

			OriginalArchetype = IntsArchetype;
			EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[3], Entities[0]);
			Result[3][0] = true;
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[5], Entities[3]);
			Result[5][3] = true;
			Result[5][0] = true; // grandparent
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[7], Entities[3]);
			Result[7][3] = true;
			Result[7][0] = true; // grandparent
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[6], Entities[5]);
			Result[6][5] = true;
			Result[6][3] = true; // grandparent
			Result[6][0] = true; // grand-grandparent

			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[8], Entities[0]);
			Result[8][0] = true;

			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[4], Entities[1]);
			Result[4][1] = true;

			NoRelation = 2;
			SingleChildParent = 1;
			SingleChildChild = 4;
			

			// we now have the following hierarchy, indexing Entities array
			// 0				1				2 - Not in hierarchy
			// +---	3			+- 4
			// |	+ 	5
			// |	|	+	 6
			// |	|
			// |	+ 	7
			// |
			// +---	8
			//

			return true;
		}
	};

	struct FChildOf : FComplexHierarchy
	{
		virtual bool InstantTest() override
		{
			// no relation
			for (int32 EntityIndex = 0; EntityIndex < Entities.Num(); ++EntityIndex)
			{
				if (EntityIndex != NoRelation)
				{
					AITEST_FALSE("NoRelation is a child of another entity", RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[NoRelation], Entities[EntityIndex]));
					AITEST_FALSE("NoRelation is a parent to another entity", RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[EntityIndex], Entities[NoRelation]));
				}
			}

			// simple 1-1 hierarchy
			AITEST_FALSE("(NOT) SingleChildParent is a child of SingleChildChild", RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[SingleChildParent], Entities[SingleChildChild]));
			AITEST_TRUE("SingleChildChild is a child of SingleChildParent", RelationshipManagerPtr->IsSubjectOfRelation(RelationTypeHandle, Entities[SingleChildChild], Entities[SingleChildParent]));

			// We'll test everything else in bulk. Here's what we want to see, rows are "children", columns the result of IsSubjectOfRelation
			// note that Result contains cumulative results, so we test indirect relations too
			for (int32 ChildIndex = 0; ChildIndex < Entities.Num(); ++ChildIndex)
			{
				for (int32 ParentIndex = 0; ParentIndex < Entities.Num(); ++ParentIndex)
				{
					AITEST_EQUAL("Child is a child", RelationshipManagerPtr->IsSubjectOfRelationRecursive(RelationTypeHandle, Entities[ChildIndex], Entities[ParentIndex]), Result[ChildIndex][ParentIndex]);
				}
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FChildOf, "System.Mass.Relations.Hierarchy.ChildOf");

	/** Testing individual policies being applied properly when relation object gets destroyed. */
	struct FObjectDestroyedPolicy : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
			const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, SubjectEntity, ObjectEntity);

			EntityManager->DestroyEntity(ObjectEntity);
			EntityManager->FlushCommands();
			AITEST_TRUE("The source entity is destroyed along with the relation object", EntityManager->IsEntityValid(SubjectEntity) == false);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FObjectDestroyedPolicy, "System.Mass.Relations.Policy.OnObjectDestroyed");

	struct FSubjectDestroyedPolicy : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;
			{
				FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestRelationA");
				LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::Destroy;

				const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

				const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, SubjectEntity, ObjectEntity);

				EntityManager->DestroyEntity(SubjectEntity);
				EntityManager->FlushCommands();
				AITEST_TRUE("The object entity is destroyed along with the relation source", EntityManager->IsEntityValid(ObjectEntity) == false);
			}
			{
				FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestRelationB");
				LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::CleanUp;
				const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

				const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, SubjectEntity, ObjectEntity);

				EntityManager->DestroyEntity(SubjectEntity);
				EntityManager->FlushCommands();
				AITEST_TRUE("The object entity is still valid once the relation source gets destroyed", EntityManager->IsEntityValid(ObjectEntity));
				// The relation has been cleaned up
				AITEST_TRUE("The object entity is still valid once the the relation source gets destroyed", EntityManager->IsEntityValid(ObjectEntity));
				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, ObjectEntity);
				AITEST_EQUAL("Expected remaining sources count", Subjects.Num(), 0);
			}
			{
				FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestRelationC");
				LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::Splice;
				// not that for this test to work the Object's Destruction policy can't be Destroy
				// since then the LeafEntity (created below) will be destroyed as part of SubjectEntity's destruction
				// before the new, patched relation gets created
				LocalRelationTraits.RoleTraits[static_cast<int32>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::Splice;
				const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

				const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				const FMassEntityHandle LeafEntity = EntityManager->CreateEntity(OriginalArchetype);
				const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, SubjectEntity, ObjectEntity);
				RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, LeafEntity, SubjectEntity);

				EntityManager->DestroyEntity(SubjectEntity);
				EntityManager->FlushCommands();
				AITEST_TRUE("The object entity is still valid once the relation source gets destroyed", EntityManager->IsEntityValid(ObjectEntity));
				AITEST_TRUE("The leaf entity is still valid once the relation source gets destroyed", EntityManager->IsEntityValid(LeafEntity));
				TArray<FMassEntityHandle> Subjects = RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, ObjectEntity);
				AITEST_EQUAL("Expected remaining sources count", Subjects.Num(), 1);
				AITEST_EQUAL("The leaf entity is the child now", Subjects[0], LeafEntity);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSubjectDestroyedPolicy, "System.Mass.Relations.Policy.OnSubjectDestroyed");

	struct FDestroyEverythingPolicy : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;
			{
				const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, SubjectEntity, ObjectEntity);

				EntityManager->DestroyEntity(ObjectEntity);
				EntityManager->FlushCommands();
				AITEST_TRUE("The source entity is destroyed along with the relation object", EntityManager->IsEntityValid(SubjectEntity) == false);
			}
			{
				const FMassEntityHandle SubjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				const FMassEntityHandle ObjectEntity = EntityManager->CreateEntity(OriginalArchetype);
				RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, SubjectEntity, ObjectEntity);

				EntityManager->DestroyEntity(SubjectEntity);
				EntityManager->FlushCommands();
				AITEST_TRUE("The source entity is destroyed along with the relation object", EntityManager->IsEntityValid(SubjectEntity) == false);
			}


			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FDestroyEverythingPolicy, "System.Mass.Relations.Policy.DestroyEverything");

	struct FHierarchyTrivialCycle : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			const FMassEntityHandle Entity = EntityManager->CreateEntity(OriginalArchetype);
			{
				MASS_SCOPED_ENSURE_TEST("subject and object to be different", 1);
				RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entity, Entity);
			}
			const FMassArchetypeHandle FinalArchetype = EntityManager->GetArchetypeForEntity(Entity);
			AITEST_TRUE("The entity has not changed archetypes", FinalArchetype == OriginalArchetype);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FHierarchyTrivialCycle, "System.Mass.Relations.Hierarchy.TrivialCycle");

#if 0
	// this test is disabled for now, no elaborate cycle detection in place yet
	struct FHierarchyCycle : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 3;
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[1], Entities[0]);
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[2], Entities[1]);
			{
				// this relation registration should fail as it would create a cycle
				MASS_SCOPED_ENSURE_TEST("result in a cycle", 1);
				RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[0], Entities[2]);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FHierarchyCycle, "System.Mass.Relations.Hierarchy.Cycle");
#endif

	struct FDestructionPolicy : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			UScriptStruct* DestroyChildOnParentDestructionRelation = MakeRelationType("DestroyChildOnParentDestruction");
			FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription(DestroyChildOnParentDestructionRelation);
			LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Object)].DestructionPolicy = ERemovalPolicy::Destroy;

			FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, 2, Entities);

			RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[1], Entities[0]);
			EntityManager->FlushCommands();

			EntityManager->DestroyEntity(Entities[0]);

			EntityManager->FlushCommands();
			AITEST_TRUE("Destroying the parent destroys the child.", EntityManager->IsEntityActive(Entities[1]) == false);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FDestructionPolicy, "System.Mass.Relations.DestructionPolicy");

	struct FExclusivity : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			if (FEntityBuilder::GetDefaultValueOfForceDeferredCommit())
			{
				// this test assumes synchronous EntityBuilder commits. Bailing out
				// to avoid generating false positives
				return true;
			}

			constexpr int32 NumEntities = 3;
			constexpr int32 OriginalParentIndex = 0;
			constexpr int32 ChildIndex = 1;
			constexpr int32 NewParentIndex = 2;
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			UScriptStruct* DestroyChildOnParentDestructionRelation = MakeRelationType("DestroyChildOnParentDestruction");
			FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription(DestroyChildOnParentDestructionRelation);
			LocalRelationTraits.RoleTraits[static_cast<uint8>(ERelationRole::Subject)].DestructionPolicy = ERemovalPolicy::Destroy;

			FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

			RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[ChildIndex], Entities[OriginalParentIndex]);
			// swapping parent
			RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[ChildIndex], Entities[NewParentIndex]);

			AITEST_TRUE("Original parent has no children", RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, Entities[OriginalParentIndex]).Num() == 0);
			AITEST_TRUE("New parent has exactly one child", RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, Entities[NewParentIndex]).Num() == 1);
			AITEST_TRUE("New parent's child matches expectations", RelationshipManagerPtr->GetRelationSubjects(LocalRelationTypeHandle, Entities[NewParentIndex])[0] == Entities[ChildIndex]);
			AITEST_TRUE("The Child has parents", RelationshipManagerPtr->GetRelationObjects(LocalRelationTypeHandle, Entities[ChildIndex]).Num() > 0);
			AITEST_TRUE("The Child knows about the new parent", RelationshipManagerPtr->GetRelationObjects(LocalRelationTypeHandle, Entities[ChildIndex])[0] == Entities[NewParentIndex]);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FExclusivity, "System.Mass.Relations.Exclusivity");

	struct FInvalidRelations : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			if (FEntityBuilder::GetDefaultValueOfForceDeferredCommit())
			{
				// this test assumes synchronous EntityBuilder commits. Bailing out
				// to avoid generating false positives
				return true;
			}

			constexpr int32 NumEntities = 3;
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			UScriptStruct* DestroyChildOnParentDestructionRelation = MakeRelationType("DestroyChildOnParentDestruction");
			FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription(DestroyChildOnParentDestructionRelation);
			const FTypeHandle LocalRelationTypeHandle = EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, NumEntities, Entities);

			RelationshipManagerPtr->CreateRelationInstance(LocalRelationTypeHandle, Entities[0], Entities[1]);
			EntityManager->FlushCommands();

			// try creating relations with mismatching pairs
			{
				MASS_SCOPED_ENSURE_TEST("Relation Objects count need match Subjects", 1);
				TArray<FMassEntityHandle> RelationInstances = RelationshipManagerPtr->CreateRelationInstances(LocalRelationTypeHandle
					, TArrayView<FMassEntityHandle>(&Entities[0], 1), TArrayView<FMassEntityHandle>(&Entities[1], 2));
				AITEST_EQUAL("A: Mismatching relation instances created count", RelationInstances.Num(), 0);
			}
			{
				TArray<FMassEntityHandle> Subjects;
				TArray<FMassEntityHandle> Objects;

				MASS_SCOPED_ENSURE_TEST("Relation needs a valid Subject entity", 1);
				Subjects.Add(FMassEntityHandle());
				Objects.Add(Entities[0]);

				MASS_SCOPED_ENSURE_TEST("Relation needs a valid Object entity", 1);
				Subjects.Add(Entities[0]);
				Objects.Add(FMassEntityHandle());
				
				MASS_SCOPED_ENSURE_TEST("Hierarchical relation requires the subject and object to be different", 1);
				Subjects.Add(Entities[0]);
				Objects.Add(Entities[0]);

				// this one we expect to succeed:
				Subjects.Add(Entities[1]);
				Objects.Add(Entities[2]);

				MASS_SCOPED_ENSURE_TEST("Relation between the two entities already exists", 1);
				Subjects.Add(Entities[0]);
				Objects.Add(Entities[1]);

				TArray<FMassEntityHandle> RelationInstances = RelationshipManagerPtr->CreateRelationInstances(LocalRelationTypeHandle, Subjects, Objects);
				AITEST_EQUAL("Expected number of valid relation instances created", RelationInstances.Num(), 1);

				FMassRelationFragment* RelationFragment = EntityManager->GetFragmentDataPtr<FMassRelationFragment>(RelationInstances[0]);
				AITEST_NOT_NULL(TEXT("The created relation entity has the relation fragment"), RelationFragment);
				AITEST_TRUE("The created relation instance has the expected subject", RelationFragment->Subject == Entities[1]);
				AITEST_TRUE("The created relation instance has the expected object", RelationFragment->Object == Entities[2]);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FInvalidRelations, "System.Mass.Relations.InvalidRelations");

	struct FReRegisteringRelationType : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			{
				FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentA");
				EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));
			}
			{
				MASS_SCOPED_ENSURE_TEST("Modifying relationship after registration done is not supported", 1);
				FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentA");
				// make it different
				LocalRelationTraits.bHierarchical = !LocalRelationTraits.bHierarchical;
				EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));
			}
			{
				MASS_SCOPED_ENSURE_TEST("Modifying relationship after registration done is not supported", 1);
				FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentB");
				// register with existing name
				EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));
			}
			{
				MASS_SCOPED_ENSURE_TEST("Modifying relationship after registration done is not supported", 1);
				FRelationTypeTraits LocalRelationTraits = CreateRelationTypeDescription("TestFragmentA");
				// register existing traits - source and object elements are supposed to be unique
				EntityManager->GetTypeManager().RegisterType(MoveTemp(LocalRelationTraits));
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FReRegisteringRelationType, "System.Mass.Relations.Type.Registration");

	struct FSymmetricCleanup : FRelationTestBase
	{
		virtual void TweakRelation(FRelationTypeTraits& TempRelationTraits) override
		{
			TempRelationTraits.RoleTraits[0].DestructionPolicy = ERemovalPolicy::CleanUp;
			TempRelationTraits.RoleTraits[1].DestructionPolicy = ERemovalPolicy::CleanUp;
		}

		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, 2, Entities);

			constexpr int32 ObjectIndex = 1;
			constexpr int32 SubjecIndex = 0;
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, Entities[SubjecIndex], Entities[ObjectIndex]);

			TArray<FMassEntityHandle> SubjectEntities = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Entities[ObjectIndex]);
			AITEST_EQUAL("Expected number of object's subjects", SubjectEntities.Num(), 1);
			AITEST_TRUE("Object's subject meets expectations", SubjectEntities[0] == Entities[SubjecIndex]);

			TArray<FMassEntityHandle> ObjectEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, Entities[SubjecIndex]);
			AITEST_EQUAL("Expected number of subject's objects", ObjectEntities.Num(), 1);
			AITEST_TRUE("Subject's object meets expectations", ObjectEntities[0] == Entities[ObjectIndex]);

			EntityManager->DestroyEntity(Entities[ObjectIndex]);

			ObjectEntities = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, Entities[SubjecIndex]);
			AITEST_EQUAL("After object's destruction: Expected number of subject's objects", ObjectEntities.Num(), 0);
			
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSymmetricCleanup, "System.Mass.Relations.SymmetricCleanup");

	struct FHasRelationSubjectsAndObjects : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, 3, Entities);

			const FMassEntityHandle ChildA = Entities[0];
			const FMassEntityHandle ChildB = Entities[1];
			const FMassEntityHandle Parent = Entities[2];

			// Before any relations: nothing should exist
			AITEST_FALSE("Parent has no subjects before relations", RelationshipManagerPtr->HasRelationSubjects(RelationTypeHandle, Parent));
			AITEST_FALSE("ChildA has no objects before relations", RelationshipManagerPtr->HasRelationObjects(RelationTypeHandle, ChildA));
			AITEST_FALSE("ChildA has no subjects before relations", RelationshipManagerPtr->HasRelationSubjects(RelationTypeHandle, ChildA));

			// Create ChildA -> Parent
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, ChildA, Parent);
			EntityManager->FlushCommands();

			AITEST_TRUE("Parent has subjects after first relation", RelationshipManagerPtr->HasRelationSubjects(RelationTypeHandle, Parent));
			AITEST_TRUE("ChildA has objects after first relation", RelationshipManagerPtr->HasRelationObjects(RelationTypeHandle, ChildA));
			AITEST_FALSE("ChildA has no subjects (it is a leaf)", RelationshipManagerPtr->HasRelationSubjects(RelationTypeHandle, ChildA));
			AITEST_FALSE("Parent has no objects (it is a root)", RelationshipManagerPtr->HasRelationObjects(RelationTypeHandle, Parent));

			// Create ChildB -> Parent (second child)
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, ChildB, Parent);
			EntityManager->FlushCommands();

			AITEST_TRUE("Parent still has subjects after second relation", RelationshipManagerPtr->HasRelationSubjects(RelationTypeHandle, Parent));
			AITEST_TRUE("ChildB has objects after relation", RelationshipManagerPtr->HasRelationObjects(RelationTypeHandle, ChildB));

			// Destroy ChildA -> Parent
			RelationshipManagerPtr->DestroyRelationInstance(RelationTypeHandle, ChildA, Parent);
			EntityManager->FlushCommands();

			AITEST_TRUE("Parent still has subjects (ChildB remains)", RelationshipManagerPtr->HasRelationSubjects(RelationTypeHandle, Parent));
			AITEST_FALSE("ChildA has no objects after relation destroyed", RelationshipManagerPtr->HasRelationObjects(RelationTypeHandle, ChildA));

			// Destroy ChildB -> Parent
			RelationshipManagerPtr->DestroyRelationInstance(RelationTypeHandle, ChildB, Parent);
			EntityManager->FlushCommands();

			AITEST_FALSE("Parent has no subjects after all relations destroyed", RelationshipManagerPtr->HasRelationSubjects(RelationTypeHandle, Parent));
			AITEST_FALSE("ChildB has no objects after relation destroyed", RelationshipManagerPtr->HasRelationObjects(RelationTypeHandle, ChildB));

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FHasRelationSubjectsAndObjects, "System.Mass.Relations.HasRelationSubjectsAndObjects");

	struct FForEachRelationSubjectAndObject : FRelationTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassArchetypeHandle OriginalArchetype = IntsArchetype;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(OriginalArchetype, 4, Entities);

			const FMassEntityHandle ChildA = Entities[0];
			const FMassEntityHandle ChildB = Entities[1];
			const FMassEntityHandle ChildC = Entities[2];
			const FMassEntityHandle Parent = Entities[3];

			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, ChildA, Parent);
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, ChildB, Parent);
			RelationshipManagerPtr->CreateRelationInstance(RelationTypeHandle, ChildC, Parent);
			EntityManager->FlushCommands();

			// ForEachRelationSubject: enumerate children of Parent
			{
				TArray<FMassEntityHandle> CollectedSubjects;
				RelationshipManagerPtr->ForEachRelationSubject(RelationTypeHandle, Parent,
					[&CollectedSubjects](FMassEntityHandle Entity)
					{
						CollectedSubjects.Add(Entity);
					});

				TArray<FMassEntityHandle> ExpectedSubjects = RelationshipManagerPtr->GetRelationSubjects(RelationTypeHandle, Parent);
				AITEST_EQUAL("ForEachRelationSubject count matches GetRelationSubjects", CollectedSubjects.Num(), ExpectedSubjects.Num());

				// Verify same set of entities (order may differ)
				for (const FMassEntityHandle& Expected : ExpectedSubjects)
				{
					AITEST_TRUE("ForEachRelationSubject contains expected entity", CollectedSubjects.Contains(Expected));
				}
			}

			// ForEachRelationObject: enumerate parents of ChildA
			{
				TArray<FMassEntityHandle> CollectedObjects;
				RelationshipManagerPtr->ForEachRelationObject(RelationTypeHandle, ChildA,
					[&CollectedObjects](FMassEntityHandle Entity)
					{
						CollectedObjects.Add(Entity);
					});

				TArray<FMassEntityHandle> ExpectedObjects = RelationshipManagerPtr->GetRelationObjects(RelationTypeHandle, ChildA);
				AITEST_EQUAL("ForEachRelationObject count matches GetRelationObjects", CollectedObjects.Num(), ExpectedObjects.Num());

				for (const FMassEntityHandle& Expected : ExpectedObjects)
				{
					AITEST_TRUE("ForEachRelationObject contains expected entity", CollectedObjects.Contains(Expected));
				}
			}

			// ForEach on entity with no relations: callback should not fire
			{
				int32 CallCount = 0;
				RelationshipManagerPtr->ForEachRelationSubject(RelationTypeHandle, ChildA,
					[&CallCount](FMassEntityHandle)
					{
						++CallCount;
					});
				AITEST_EQUAL("ForEachRelationSubject on leaf fires zero callbacks", CallCount, 0);
			}
			{
				int32 CallCount = 0;
				RelationshipManagerPtr->ForEachRelationObject(RelationTypeHandle, Parent,
					[&CallCount](FMassEntityHandle)
					{
						++CallCount;
					});
				AITEST_EQUAL("ForEachRelationObject on root fires zero callbacks", CallCount, 0);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FForEachRelationSubjectAndObject, "System.Mass.Relations.ForEachRelationSubjectAndObject");

}// UE::Mass::Test::Relations

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
