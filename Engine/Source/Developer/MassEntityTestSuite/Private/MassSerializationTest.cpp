// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassProcessingTypes.h"
#include "MassEntityTestTypes.h"
#include "MassExecutor.h"
#include "MassEntityView.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

struct FTestStructTracker : FStructTracker
{
	FTestStructTracker()
		: FStructTracker(FMassFragment::StaticStruct())
	{}

	template<typename T>
	int32 Add()
	{
		return StructTypesList.AddUnique(T::StaticStruct());
	}

	int32 Add(const UScriptStruct& Type)
	{
		return StructTypesList.AddUnique(&Type);
	}
};

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//
namespace FMassSerializationTest
{

//-----------------------------------------------------------------------------
// BitSet serialization
//-----------------------------------------------------------------------------
struct FSerialization_BitSet : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentBitSet BitSet;

		BitSet.Add<FTestFragment_Float>();
		BitSet.Add<FTestFragment_Bool>();
		BitSet.Add<FTestFragment_Int>();

		TArray<uint8> Data;
		FMemoryWriter Writer(Data);

		BitSet.Serialize(Writer);

		FMemoryReader Reader(Data);
		FMassFragmentBitSet NewBitSet;
		NewBitSet.Serialize(Reader);

		AITEST_TRUE("Saving and loading bitset of a given type should result in equivalent bitsets", BitSet.IsEquivalent(NewBitSet));
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSerialization_BitSet, "System.Mass.Serialization.BitSet.Trivial");

#if 0 // this test is disable for the time being, the functionality being tested has not been fully implemented

/*
 * Here's what this test does:
 * 1. creates a fake "old" tracker that knows about three types, and then creates a bit container having two of those 
 *	set to true 
 * 2. serializes this 2-out-of-3 bit container.
 * 3. creates instance of the FMassFragmentBitSet type that hosts a tracker that knows about all the fragment structs,
 *	including the ones used by "old fake" tracker.
 * 4. reads the "outdated 2-out-of-3" data into an instance of FMassFragmentBitSet
 * The point is that the order and indices of types in "old" tracker and the FMassFragmentBitSet's tracker are totally
 * different, but loading patches it up.
*/
struct FSerialization_BitSetLoadOutdated : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		FTestStructTracker TestTracker;
		const int32 FloatIndex = TestTracker.Add<FTestFragment_Float>();
		TestTracker.Add<FTestFragment_Bool>();
		const int32 IntIndex = TestTracker.Add<FTestFragment_Int>();
		FStructTypeBitSet::FBitSetContainer TestBitArray;
		TestBitArray.AddAtIndex(FloatIndex);
		TestBitArray.AddAtIndex(IntIndex);

		// create a bitset we're expected to get while loading TestBitArray's contents into FMassFragmentBitSet
		FMassFragmentBitSet RegularBitSet;
		RegularBitSet.Add<FTestFragment_Float>();
		RegularBitSet.Add<FTestFragment_Int>();

		// save
		TArray<uint8> Data;
		FMemoryWriter Writer(Data);
		TestTracker.Serialize(Writer, TestBitArray);

		// load
		FMemoryReader Reader(Data);
		FMassFragmentBitSet SerializedBitSet;
		SerializedBitSet.Serialize(Reader);

		AITEST_TRUE("Loading data serialized by the TestStructTracker should produce in a BitSet equivalent to the one explicitly created"
			, RegularBitSet.IsEquivalent(SerializedBitSet));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSerialization_BitSetLoadOutdated, "System.Mass.Serialization.BitSet.LoadOutdated");

#endif // temp disabling of the System.Mass.Serialization.BitSet.LoadOutdated test

struct FSerialization_BitSetOverride : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		// create a bitset we're expected to get while loading TestBitArray's contents into FMassFragmentBitSet
		FMassFragmentBitSet RegularBitSet;
		RegularBitSet.Add<FTestFragment_Float>();;
		RegularBitSet.Add<FTestFragment_Int>();

		// save
		TArray<uint8> Data;
		FMemoryWriter Writer(Data);
		RegularBitSet.Serialize(Writer);

		// load
		FMemoryReader Reader(Data);
		FMassFragmentBitSet SerializedBitSet;
		// KEY STEP - add some crap to pollute the bitset
		SerializedBitSet.Add<FTestFragment_Large>();
		SerializedBitSet.Add<FTestFragment_Array>();

		SerializedBitSet.Serialize(Reader);

		AITEST_TRUE("The original contents of SerializedBitSet should have been erased", RegularBitSet.IsEquivalent(SerializedBitSet));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSerialization_BitSetOverride, "System.Mass.Serialization.BitSet.OverrideExisting");

} // FMassSerializationTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
