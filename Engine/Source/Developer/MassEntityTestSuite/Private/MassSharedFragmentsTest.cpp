// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTypes.h"
#include "MassEntityTestTypes.h"
#include "MassArchetypeData.h"
#include "MassArchetypeTypes.h"
#include "Algo/RandomShuffle.h"
#include "MassObserverNotificationTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassEntityTest
{
//-----------------------------------------------------------------------------
// Helpers
//-----------------------------------------------------------------------------
template<typename TMassSharedFragmentType>
const TMassSharedFragmentType* GetConstSharedFragmentPtr(const FMassArchetypeSharedFragmentValues& Values)
{
	FConstSharedStruct SharedStruct = Values.GetConstSharedFragmentStruct(TMassSharedFragmentType::StaticStruct());
	return SharedStruct.IsValid() ? SharedStruct.GetPtr<const TMassSharedFragmentType>() : nullptr;
}

template<typename TMassSharedFragmentType>
TMassSharedFragmentType* GetMutableSharedFragmentPtr(FMassArchetypeSharedFragmentValues& Values)
{
	FSharedStruct SharedStruct = Values.GetSharedFragmentStruct(TMassSharedFragmentType::StaticStruct());
	return SharedStruct.IsValid() ? SharedStruct.GetPtr<TMassSharedFragmentType>() : nullptr;
}

bool HasExactSharedFragmentTypesMatch(const FMassArchetypeSharedFragmentValues& Values, const FMassSharedFragmentBitSet SharedFragmentBitSet)
{
	return Values.GetBitSet().Get<FMassSharedFragmentBitSet>() == SharedFragmentBitSet;
}

bool HasExactConstSharedFragmentTypesMatch(const FMassArchetypeSharedFragmentValues& Values, const FMassConstSharedFragmentBitSet SharedFragmentBitSet)
{
	return Values.GetBitSet().Get<FMassConstSharedFragmentBitSet>() == SharedFragmentBitSet;
}

//-----------------------------------------------------------------------------
// FMassArchetypeSharedFragmentValues Tests
//-----------------------------------------------------------------------------
struct FSharedFragmentValues_Create : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 59;
		FMassArchetypeSharedFragmentValues Values;

		{
			FTestSharedFragment_Int FragmentInstance(TestIntValue);
			FSharedStruct SharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
			Values.Add(SharedFragmentInstance);
		}

		const FTestSharedFragment_Int* ConstInstance = GetConstSharedFragmentPtr<FTestSharedFragment_Int>(Values);
		FTestSharedFragment_Int* NonConstInstance = GetMutableSharedFragmentPtr<FTestSharedFragment_Int>(Values);

		AITEST_NULL("Fetching fragment as a const shared fragment should fail", ConstInstance);
		AITEST_NOT_NULL("Fetching fragment as a shared fragment should not fail", NonConstInstance);
		AITEST_EQUAL("The fetched value should match the expectations", NonConstInstance->Value, TestIntValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_Create, "System.Mass.SharedFragments.CreateValue");

struct FSharedFragmentValues_CreateConst : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 59;
		FMassArchetypeSharedFragmentValues Values;

		{
			FConstSharedStruct SharedFragmentInstance = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue);
			Values.Add(SharedFragmentInstance);
		}

		const FTestConstSharedFragment_Int* ConstInstance = GetConstSharedFragmentPtr<FTestConstSharedFragment_Int>(Values);
		FTestConstSharedFragment_Int* NonConstInstance = GetMutableSharedFragmentPtr<FTestConstSharedFragment_Int>(Values);

		AITEST_NULL("Fetching fragment as a shared fragment should fail", NonConstInstance);
		AITEST_NOT_NULL("Fetching fragment as a const shared fragment should not fail", ConstInstance);
		AITEST_EQUAL("The fetched value should match the expectations", ConstInstance->Value, TestIntValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_CreateConst, "System.Mass.SharedFragments.CreateConstValue");

struct FSharedFragmentValues_Contains : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 31;
		constexpr float TestFloatValue = 63.f;
		FMassArchetypeSharedFragmentValues Values;

		AITEST_FALSE("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests"
			, Values.ContainsType(FTestSharedFragment_Int::StaticStruct()));
		AITEST_FALSE("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests"
			, Values.ContainsType<FTestSharedFragment_Int>());

		{
			FSharedStruct SharedFragmentInstance = FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue);
			Values.Add(SharedFragmentInstance);
		}

		AITEST_TRUE("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests"
			, Values.ContainsType(FTestSharedFragment_Int::StaticStruct()));
		AITEST_TRUE("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests"
			, Values.ContainsType<FTestSharedFragment_Int>());

		{
			FSharedStruct SharedFragmentInstance = FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue);
			Values.Add(SharedFragmentInstance);
		}
		AITEST_TRUE("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests"
			, Values.ContainsType(FTestSharedFragment_Float::StaticStruct()));
		AITEST_TRUE("Empty FMassArchetypeSharedFragmentValues should fail ContainsType tests"
			, Values.ContainsType<FTestSharedFragment_Float>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_Contains, "System.Mass.SharedFragments.Contains");

struct FSharedFragmentValues_Append : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 31;
		constexpr float TestFloatValue = 63.f;
		
		FMassArchetypeSharedFragmentValues ValuesNonConstInt;
		ValuesNonConstInt.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
		FMassArchetypeSharedFragmentValues ValuesNonConstFloat;
		ValuesNonConstFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));
		FMassArchetypeSharedFragmentValues ValuestNonConstIntFloat;
		ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
		ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));

		// `Appending` int/float values to an new Values instance should result in the same result as `Adding` them
		{
			FMassArchetypeSharedFragmentValues Values;
			Values.Append(ValuesNonConstInt);
			AITEST_TRUE("#1 Append results should match expectations", Values.HasSameValues(ValuesNonConstInt));
			Values.Append(ValuesNonConstFloat);
			AITEST_TRUE("#2 Append results should match expectations", Values.HasSameValues(ValuestNonConstIntFloat));
		}
		{
			FMassArchetypeSharedFragmentValues Values;
			Values.Append(ValuesNonConstFloat);
			AITEST_TRUE("#3 Append results should match expectations", Values.HasSameValues(ValuesNonConstFloat));
			Values.Append(ValuesNonConstInt);
			AITEST_TRUE("#4 Append results should match expectations", Values.HasSameValues(ValuestNonConstIntFloat));
		}

		FMassArchetypeSharedFragmentValues ValuesConstInt;
		ValuesConstInt.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
		FMassArchetypeSharedFragmentValues ValuesConstFloat;
		ValuesConstFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));
		FMassArchetypeSharedFragmentValues ValuestConstIntFloat;
		ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
		ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));

		{
			FMassArchetypeSharedFragmentValues Values;
			Values.Append(ValuesConstInt);
			AITEST_TRUE("#5 Append results should match expectations", Values.HasSameValues(ValuesConstInt));
			Values.Append(ValuesConstFloat);
			AITEST_TRUE("#6 Append results should match expectations", Values.HasSameValues(ValuestConstIntFloat));
		}
		{
			FMassArchetypeSharedFragmentValues Values;
			Values.Append(ValuesConstFloat);
			AITEST_TRUE("#7 Append results should match expectations", Values.HasSameValues(ValuesConstFloat));
			Values.Append(ValuesConstInt);
			AITEST_TRUE("#8 Append results should match expectations", Values.HasSameValues(ValuestConstIntFloat));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_Append, "System.Mass.SharedFragments.Append");

struct FSharedFragmentValues_Remove : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 31;
		constexpr float TestFloatValue = 63.f;

		{
			FMassArchetypeSharedFragmentValues ValuesNonConstInt;
			ValuesNonConstInt.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
			FMassArchetypeSharedFragmentValues ValuesNonConstFloat;
			ValuesNonConstFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));
			FMassArchetypeSharedFragmentValues ValuestNonConstIntFloat;
			ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
			ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));

			{
				FMassArchetypeSharedFragmentValues Values = ValuestNonConstIntFloat;
				AITEST_TRUE("Assignment should result in same values", Values.HasSameValues(ValuestNonConstIntFloat));

				// removing just the Int shared fragment
				Values.Remove(ValuesNonConstInt);
				AITEST_TRUE("#1 Removal results should match expectations", Values.HasSameValues(ValuesNonConstFloat));
			}
			{
				FMassArchetypeSharedFragmentValues Values = ValuestNonConstIntFloat;
				// removing just the Float shared fragment
				Values.Remove(ValuesNonConstFloat);
				AITEST_TRUE("#2 Removal results should match expectations", Values.HasSameValues(ValuesNonConstInt));
			}
		}
		{
			FMassArchetypeSharedFragmentValues ValuesConstInt;
			ValuesConstInt.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
			FMassArchetypeSharedFragmentValues ValuesConstFloat;
			ValuesConstFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));
			FMassArchetypeSharedFragmentValues ValuestConstIntFloat;
			ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
			ValuestConstIntFloat.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(TestFloatValue));

			{
				FMassArchetypeSharedFragmentValues Values = ValuestConstIntFloat;
				AITEST_TRUE("Assignment should result in same values", Values.HasSameValues(ValuestConstIntFloat));

				// removing just the Int shared fragment
				Values.Remove(ValuesConstInt);
				AITEST_TRUE("#3 Removal results should match expectations", Values.HasSameValues(ValuesConstFloat));
			}
			{
				FMassArchetypeSharedFragmentValues Values = ValuestConstIntFloat;
				// removing just the Float shared fragment
				Values.Remove(ValuesConstFloat);
				AITEST_TRUE("#4 Removal results should match expectations", Values.HasSameValues(ValuesConstInt));
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_Remove, "System.Mass.SharedFragments.Remove");

struct FSharedFragmentValues_Hash : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 31;
		constexpr float TestFloatValue = 63.f;

		FMassArchetypeSharedFragmentValues ValuestNonConstIntFloat;
		ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
		ValuestNonConstIntFloat.Add(FSharedStruct::Make<FTestSharedFragment_Float>(TestFloatValue));

		MASS_SCOPED_ENSURE_TEST("Expecting the containers to be sorted", 1);
		const uint32 EmptyHash = ValuestNonConstIntFloat.CalculateHash();
		AITEST_EQUAL("Expecting unsorted collection hashing to result in 0", EmptyHash, 0u);

		ValuestNonConstIntFloat.Sort();
		const uint32 ValidHash = ValuestNonConstIntFloat.CalculateHash();
		AITEST_NOT_EQUAL("Expecting sorted collection hashing to result in non 0", ValidHash, 0u);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_Hash, "System.Mass.SharedFragments.Hash");

struct FSharedFragment_ForEach : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 NumSharedFragments = 4;
		TStaticArray<int32, NumSharedFragments> TestInitValues = { 9, 1, 12, 13};
		
		for (int32 InitValue : TestInitValues)
		{
			EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(InitValue); //-V530
		}

		TArray<int32> Results;
		TArray<int32> ModifiedValues;
		EntityManager->ForEachSharedFragment<FTestSharedFragment_Int>([&Results, &ModifiedValues](FTestSharedFragment_Int& SharedFragment)
		{
			Results.Add(SharedFragment.Value);
			ModifiedValues.Add(SharedFragment.Value += 100);
		});

		AITEST_EQUAL("Number of processed shared fragments", Results.Num(), NumSharedFragments);
		for (int32 InitValue : TestInitValues)
		{
			AITEST_TRUE("Read values matches init values", Results.Find(InitValue) != INDEX_NONE);
		}

		TArray<int32> MutatedResults;
		EntityManager->ForEachSharedFragment<FTestSharedFragment_Int>([&MutatedResults](const FTestSharedFragment_Int& SharedFragment)
		{
			MutatedResults.Add(SharedFragment.Value);
		});

		AITEST_EQUAL("Number of shared fragments processed in second round", Results.Num(), NumSharedFragments);
		for (int32 ModifiedValue : ModifiedValues)
		{
			AITEST_TRUE("Read values matches values set in the first round", MutatedResults.Find(ModifiedValue) != INDEX_NONE);
		}

		constexpr int32 ConditionalLimit = 10;
		TArray<int32> ConditionalResults;
		EntityManager->ForEachSharedFragmentConditional<FTestSharedFragment_Int>(
			[](FTestSharedFragment_Int& SharedFragment)
			{
				return SharedFragment.Value > ConditionalLimit;
			}
			, [&ConditionalResults](FTestSharedFragment_Int& SharedFragment)
			{
				ConditionalResults.Add(SharedFragment.Value);
			}
		);
		for (int32 Value : ConditionalResults)
		{
			AITEST_TRUE("Only the values matching the condition get processed", Value > ConditionalLimit);
		}
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_ForEach, "System.Mass.SharedFragments.ForEach");

struct FConstSharedFragment_ForEach : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 NumSharedFragments = 4;
		TStaticArray<int32, NumSharedFragments> TestInitValues = { 9, 1, 12, 13 };
		
		for (int32 InitValue : TestInitValues)
		{
			EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(InitValue); //-V530
		}

		TArray<int32> Results;
		EntityManager->ForEachConstSharedFragment<FTestConstSharedFragment_Int>([&Results](const FTestConstSharedFragment_Int& SharedFragment)
		{
			Results.Add(SharedFragment.Value);
		});

		AITEST_EQUAL("Number of processed shared fragments", Results.Num(), NumSharedFragments);
		for (int32 InitValue : TestInitValues)
		{
			AITEST_TRUE("Read values matches init values", Results.Find(InitValue) != INDEX_NONE);
		}

		constexpr int32 ConditionalLimit = 10;
		TArray<int32> ConditionalResults;
		EntityManager->ForEachConstSharedFragmentConditional<FTestConstSharedFragment_Int>(
			[](const FTestConstSharedFragment_Int& SharedFragment)
			{
				return SharedFragment.Value > ConditionalLimit;
			}
			, [&ConditionalResults](const FTestConstSharedFragment_Int& SharedFragment)
			{
				ConditionalResults.Add(SharedFragment.Value);
			}
		);
		for (int32 Value : ConditionalResults)
		{
			AITEST_TRUE("Only the values matching the condition get processed", Value > ConditionalLimit);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FConstSharedFragment_ForEach, "System.Mass.ConstSharedFragments.ForEach");

//-----------------------------------------------------------------------------
// Entity-related Tests
//-----------------------------------------------------------------------------
struct FSharedFragmentBase : public FEntityTestBase
{
	template<typename TSharedStruct, typename TSharedFragment>
	FConstStructView GetSharedFragmentView(const FMassEntityHandle EntityHandle)
	{
		if constexpr (TIsDerivedFrom<TSharedStruct, FSharedStruct>::IsDerived)
		{
			return EntityManager->GetSharedFragmentDataStruct(EntityHandle, TSharedFragment::StaticStruct());
		}
		else
		{
			return EntityManager->GetConstSharedFragmentDataStruct(EntityHandle, TSharedFragment::StaticStruct());
		}
	};

	template<typename TSharedStruct, typename TSharedFragment>
	void CreateEntities(TArray<FMassEntityHandle>& OutEntityHandles, const int32 NumToCreate, const typename TSharedFragment::FValueType TestValue)
	{
		TSharedFragment FragmentInstance(TestValue);
		FMassArchetypeSharedFragmentValues SharedIntValues;

		TSharedStruct SharedFragmentInstance = TSharedStruct::Make(FragmentInstance);
		SharedIntValues.Add(SharedFragmentInstance);

		EntityManager->BatchCreateEntities(FloatsArchetype, SharedIntValues, NumToCreate, OutEntityHandles);
	}

	template<typename TSharedStruct, typename TSharedFragment>
	void CreateEntity(FMassEntityHandle& OutEntityHandle, const typename TSharedFragment::FValueType TestValue)
	{
		TSharedFragment FragmentInstance(TestValue);
		FMassArchetypeSharedFragmentValues SharedIntValues;

		TSharedStruct SharedFragmentInstance = TSharedStruct::Make(FragmentInstance);
		SharedIntValues.Add(SharedFragmentInstance);

		OutEntityHandle = EntityManager->CreateEntity(FloatsArchetype, SharedIntValues);
	}
};

template<typename TSharedStruct, typename TSharedFragment>
struct FSharedFragment_CreateEntitesWithSharedFragment : public FSharedFragmentBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValueA = 1023;
		constexpr int32 TestIntValueB = 63;
		FMassEntityHandle EntityA, EntityB;

		CreateEntity<TSharedStruct, TSharedFragment>(EntityA, TestIntValueA);
		CreateEntity<TSharedStruct, TSharedFragment>(EntityB, TestIntValueB);

		AITEST_EQUAL("Both entities should end up in the same archetype", EntityManager->GetArchetypeForEntityUnsafe(EntityA)
			, EntityManager->GetArchetypeForEntityUnsafe(EntityB));

		/*FConstStructView SharedFragmentA, SharedFragmentB;
		if constexpr (TIsDerivedFrom<TSharedStruct, FSharedStruct>::IsDerived)
		{
			SharedFragmentA = EntityManager->GetSharedFragmentDataStruct(EntityA, FTestSharedFragment_Int::StaticStruct());
			SharedFragmentB = EntityManager->GetSharedFragmentDataStruct(EntityB, FTestSharedFragment_Int::StaticStruct());
		}
		else
		{
			SharedFragmentA = EntityManager->GetConstSharedFragmentDataStruct(EntityA, FTestSharedFragment_Int::StaticStruct());
			SharedFragmentB = EntityManager->GetConstSharedFragmentDataStruct(EntityB, FTestSharedFragment_Int::StaticStruct());
		}*/
		FConstStructView SharedFragmentA = GetSharedFragmentView<TSharedStruct, TSharedFragment>(EntityA);
		FConstStructView SharedFragmentB = GetSharedFragmentView<TSharedStruct, TSharedFragment>(EntityB);

		AITEST_TRUE("SharedFragmentA should be valid", SharedFragmentA.IsValid());
		AITEST_TRUE("SharedFragmentB should be valid", SharedFragmentB.IsValid());
		AITEST_EQUAL("SharedFragmentA should be of expected type", SharedFragmentA.GetScriptStruct(), TSharedFragment::StaticStruct());
		AITEST_EQUAL("SharedFragmentB should be of expected type", SharedFragmentB.GetScriptStruct(), TSharedFragment::StaticStruct());
		AITEST_NOT_EQUAL("SharedFragmentA and SharedFragmentB should be different instanceS", SharedFragmentA, SharedFragmentB);
		AITEST_NOT_EQUAL("SharedFragmentA and SharedFragmentB should be distinct", SharedFragmentA, SharedFragmentB);
		AITEST_EQUAL("SharedFragmentA's value should match the expected value", SharedFragmentA.Get<const TSharedFragment>().Value, TestIntValueA);
		AITEST_EQUAL("SharedFragmentB's value should match the expected value", SharedFragmentB.Get<const TSharedFragment>().Value, TestIntValueB);

		return true;
	}
};
using FSharedFragment_CreateEntitesWithNonConstSharedFragment = FSharedFragment_CreateEntitesWithSharedFragment<FSharedStruct, FTestSharedFragment_Int>;
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_CreateEntitesWithNonConstSharedFragment, "System.Mass.SharedFragments.CreateEntities");
using FSharedFragment_CreateEntitesWithConstSharedFragment = FSharedFragment_CreateEntitesWithSharedFragment<FConstSharedStruct, FTestConstSharedFragment_Int>;
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_CreateEntitesWithConstSharedFragment, "System.Mass.SharedFragments.CreateEntitiesConst");

template<typename TSharedStruct, typename TSharedFragment>
struct FSharedFragment_BatchCreateEntitesWithSharedFragment : public FSharedFragmentBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValueA = 1023;
		constexpr int32 TestIntValueB = 63;
		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();
		// we create more than one chunk can handle properly test moving entities between chunks
		const int32 EntitiesToCreateNumA = FMath::FloorToInt(float(EntitiesPerChunk) * 1.2f);
		const int32 EntitiesToCreateNumB = 1;
		constexpr int32 ExpectedNumberOfInitialChunks = 3;
		TArray<FMassEntityHandle> EntitiesA;
		TArray<FMassEntityHandle> EntitiesB;

		CreateEntities<TSharedStruct, TSharedFragment>(EntitiesA, EntitiesToCreateNumA, TestIntValueA);
		CreateEntities<TSharedStruct, TSharedFragment>(EntitiesB, EntitiesToCreateNumB, TestIntValueB);

		FMassArchetypeHandle CommonArchetype = EntityManager->GetArchetypeForEntityUnsafe(EntitiesA[0]);
		AITEST_EQUAL("All the entities should end up in the same archetype"
			, FMassArchetypeHelper::ArchetypeDataFromHandleChecked(CommonArchetype).GetNumEntities(), EntitiesToCreateNumA + EntitiesToCreateNumB);
		AITEST_EQUAL("The total number of chunks in the resulting archetype should match expectations"
			, FMassArchetypeHelper::ArchetypeDataFromHandleChecked(CommonArchetype).GetChunkCount(), ExpectedNumberOfInitialChunks);

		for (FMassEntityHandle EntityHandle : EntitiesA)
		{
			FConstStructView SharedFragment = GetSharedFragmentView<TSharedStruct, TSharedFragment>(EntityHandle);
			AITEST_TRUE("SharedFragment for entity type A should be valid", SharedFragment.IsValid());
			AITEST_EQUAL("SharedFragment for entity type A  should be of expected type", SharedFragment.GetScriptStruct(), TSharedFragment::StaticStruct());
			AITEST_EQUAL("SharedFragment's value for entity type A should match the expected value", SharedFragment.Get<const TSharedFragment>().Value, TestIntValueA);
		}

		FConstStructView SharedFragment = GetSharedFragmentView<TSharedStruct, TSharedFragment>(EntitiesB[0]);
		AITEST_TRUE("SharedFragment for entity type B should be valid", SharedFragment.IsValid());
		AITEST_EQUAL("SharedFragment for entity type B  should be of expected type", SharedFragment.GetScriptStruct(), TSharedFragment::StaticStruct());
		AITEST_EQUAL("SharedFragment's value for entity type B should match the expected value", SharedFragment.Get<const TSharedFragment>().Value, TestIntValueB);

		return true;
	}
};
using FSharedFragment_BatchCreateEntitesWithNonConstSharedFragment = FSharedFragment_BatchCreateEntitesWithSharedFragment<FSharedStruct, FTestSharedFragment_Int>;
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_BatchCreateEntitesWithNonConstSharedFragment, "System.Mass.SharedFragments.BatchCreateEntities");
using FSharedFragment_BatchCreateEntitesWithConstSharedFragment = FSharedFragment_BatchCreateEntitesWithSharedFragment<FConstSharedStruct, FTestConstSharedFragment_Int>;
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_BatchCreateEntitesWithConstSharedFragment, "System.Mass.SharedFragments.BatchCreateEntitiesConst");


struct FConstSharedFragment_AddToEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 1023;
		const FTestConstSharedFragment_Int FragmentInstance(TestIntValue);

		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

		const FTestConstSharedFragment_Int* EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		AITEST_NULL("Initially the entity is not expected to have the shared fragment", EntitySharedFragment);

		{
			MASS_SCOPED_ENSURE_TEST("shared struct requires to be registered first", 1);
			const FConstSharedStruct UnregisteredSharedFragmentInstance = FSharedStruct::Make(FragmentInstance);
			const bool bSuccessfullyAddedConstSharedFragment =  EntityManager->AddConstSharedFragmentToEntity(EntityHandle, UnregisteredSharedFragmentInstance);
			AITEST_FALSE("Adding unregistered const shared fragment struct should fail", bSuccessfullyAddedConstSharedFragment);
		}

		{
			const FConstSharedStruct& RegisteredSharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);
			const bool bSuccessfullyAddedConstSharedFragment = EntityManager->AddConstSharedFragmentToEntity(EntityHandle, RegisteredSharedFragmentInstance);
			AITEST_TRUE("Adding registered const shared fragment struct should succeed", bSuccessfullyAddedConstSharedFragment);
		}

		EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		AITEST_NOT_NULL("The entity is expected to have the shared fragment after the operation", EntitySharedFragment);
		AITEST_EQUAL("The the shared fragment is expected to store the configured value", EntitySharedFragment->Value, TestIntValue);
		AITEST_NOT_EQUAL("The the entity's new archetype is not the same as the original one", EntityManager->GetArchetypeForEntity(EntityHandle), FloatsArchetype);

		// at this point the Entity already has a shared fragment of a given type
		// now we're going to add it again and test the systems behavior, we'll be adding the same FMasSharedFragment type
		constexpr int32 DifferentTestIntValue = TestIntValue + 1;
		const FTestConstSharedFragment_Int DifferentFragmentInstance(DifferentTestIntValue);
		const FConstSharedStruct DifferentConstSharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(DifferentFragmentInstance);

		MASS_SCOPED_ENSURE_TEST("Changing shared fragment value of entities is not supported", 1);

		const bool bSuccessfullyAddedConstSharedFragment = EntityManager->AddConstSharedFragmentToEntity(EntityHandle, DifferentConstSharedFragmentInstance);
		AITEST_FALSE("Adding existing const shared fragment type should fail", bSuccessfullyAddedConstSharedFragment);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FConstSharedFragment_AddToEntity, "System.Mass.ConstSharedFragments.AddToEntity");

struct FSharedFragment_AddToEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 1023;
		const FTestSharedFragment_Int FragmentInstance(TestIntValue);
		const FSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateSharedFragment(FragmentInstance);

		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

		const FTestSharedFragment_Int* EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
		AITEST_NULL("Initially the entity is not expected to have the shared fragment", EntitySharedFragment);

		EntityManager->AddSharedFragmentToEntity(EntityHandle, SharedFragmentInstance);

		EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
		AITEST_NOT_NULL("The entity is expected to have the shared fragment after the operation", EntitySharedFragment);
		AITEST_EQUAL("The the shared fragment is expected to store the configured value", EntitySharedFragment->Value, TestIntValue);

		// at this point the Entity already has a shared fragment of a given type
		// now we're going to add it again and test the systems behavior.
		constexpr int32 DifferentTestIntValue = TestIntValue + 1;
		const FTestSharedFragment_Int DifferentFragmentInstance(DifferentTestIntValue);
		const FSharedStruct DifferentSharedFragmentInstance = EntityManager->GetOrCreateSharedFragment(DifferentFragmentInstance);

		GetTestRunner().AddExpectedError(TEXT("Changing shared fragment value of entities is not supported"), EAutomationExpectedErrorFlags::Contains, 1);

		const bool bSuccessfullyAddedSharedFragment = EntityManager->AddSharedFragmentToEntity(EntityHandle, DifferentSharedFragmentInstance);
		AITEST_FALSE("Adding existing shared fragment type should fail", bSuccessfullyAddedSharedFragment);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_AddToEntity, "System.Mass.SharedFragments.AddToEntity");

struct FConstSharedFragment_RemoveFromEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 1023;
		const FTestConstSharedFragment_Int FragmentInstance(TestIntValue);
		const FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);

		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

		const FTestConstSharedFragment_Int* EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		AITEST_NULL("Initially the entity is not expected to have the shared fragment", EntitySharedFragment);

		AITEST_FALSE("Attempt to remove shared fragment from entity that doesn't have shared fragment should return false and do nothing",
			EntityManager->RemoveConstSharedFragmentFromEntity(EntityHandle, FTestConstSharedFragment_Int::StaticStruct()));

		AITEST_TRUE("Adding shared fragment to entity should succeed",
			EntityManager->AddConstSharedFragmentToEntity(EntityHandle, SharedFragmentInstance));

		EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		AITEST_NOT_NULL("The entity is expected to have the shared fragment after the operation", EntitySharedFragment);
		AITEST_EQUAL("The the shared fragment is expected to store the configured value", EntitySharedFragment->Value, TestIntValue);

		AITEST_TRUE(
			"Removing shared fragment from entity that has the shared fragment should succeed",
			EntityManager->RemoveConstSharedFragmentFromEntity(EntityHandle, FTestConstSharedFragment_Int::StaticStruct()));

		EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		AITEST_NULL("The entity is not expected to have the shared fragment after the operation", EntitySharedFragment);
		
		AITEST_EQUAL("The the entity's new archetype is the same as the initial one", EntityManager->GetArchetypeForEntity(EntityHandle), FloatsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FConstSharedFragment_RemoveFromEntity, "System.Mass.ConstSharedFragments.RemoveFromEntity");

struct FSharedFragment_RemoveFromEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 1023;
		const FTestSharedFragment_Int FragmentInstance(TestIntValue);
		const FSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateSharedFragment(FragmentInstance);

		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

		const FTestSharedFragment_Int* EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
		AITEST_NULL("Initially the entity is not expected to have the shared fragment", EntitySharedFragment);

		AITEST_FALSE("Attempt to remove shared fragment from entity that doesn't have shared fragment should return false and do nothing",
			EntityManager->RemoveSharedFragmentFromEntity(EntityHandle, FTestSharedFragment_Int::StaticStruct()));

		AITEST_TRUE("Adding shared fragment to entity should succeed",
			EntityManager->AddSharedFragmentToEntity(EntityHandle, SharedFragmentInstance));

		EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
		AITEST_NOT_NULL("The entity is expected to have the shared fragment after the operation", EntitySharedFragment);
		AITEST_EQUAL("The the shared fragment is expected to store the configured value", EntitySharedFragment->Value, TestIntValue);

		AITEST_TRUE(
			"Removing shared fragment from entity that has the shared fragment should succeed",
			EntityManager->RemoveSharedFragmentFromEntity(EntityHandle, FTestSharedFragment_Int::StaticStruct()));

		EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
		AITEST_NULL("The entity is not expected to have the shared fragment after the operation", EntitySharedFragment);
		
		AITEST_EQUAL("The the entity's new archetype is the same as the initial one", EntityManager->GetArchetypeForEntity(EntityHandle), FloatsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_RemoveFromEntity, "System.Mass.SharedFragments.RemoveFromEntity");

// Tests that an entity's shared fragment values are preserved (not dropped) when
// RemoveFragmentListFromEntity triggers an archetype change.
struct FRemoveFragmentList_PreservesSharedFragmentValues : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 42;
		const FTestSharedFragment_Int FragmentInstance(TestIntValue);
		const FSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateSharedFragment(FragmentInstance);

		// Start with an entity that has a float fragment (FloatsArchetype)
		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

		// Add a shared fragment — entity now lives in a new archetype: {float, SharedInt}
		AITEST_TRUE("Adding shared fragment should succeed",
			EntityManager->AddSharedFragmentToEntity(EntityHandle, SharedFragmentInstance));

		const FTestSharedFragment_Int* EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
		AITEST_NOT_NULL("Entity should have the shared fragment before the remove", EntitySharedFragment);
		AITEST_EQUAL("Shared fragment should hold the configured value", EntitySharedFragment->Value, TestIntValue);

		// Remove the regular float fragment via RemoveFragmentListFromEntity — triggers an archetype change to {SharedInt}
		const UScriptStruct* FragmentsToRemove[] = { FTestFragment_Float::StaticStruct() };
		EntityManager->RemoveFragmentListFromEntity(EntityHandle, MakeArrayView(FragmentsToRemove));

		// Shared fragment must survive the archetype move
		EntitySharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityHandle);
		AITEST_NOT_NULL("Entity should still have the shared fragment after removing the regular fragment", EntitySharedFragment);
		AITEST_EQUAL("Shared fragment value should be unchanged after the archetype move", EntitySharedFragment->Value, TestIntValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRemoveFragmentList_PreservesSharedFragmentValues, "System.Mass.SharedFragments.RemoveFragmentList.PreservesSharedFragmentValues");

// Tests the typed template overload RemoveConstSharedFragmentFromEntity<T>().
struct FConstSharedFragment_RemoveFromEntity_Typed : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 77;
		const FTestConstSharedFragment_Int FragmentInstance(TestIntValue);
		const FConstSharedStruct SharedFragmentInstance = EntityManager->GetOrCreateConstSharedFragment(FragmentInstance);

		const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(FloatsArchetype);

		AITEST_TRUE("Adding const shared fragment should succeed",
			EntityManager->AddConstSharedFragmentToEntity(EntityHandle, SharedFragmentInstance));

		const FTestConstSharedFragment_Int* EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		AITEST_NOT_NULL("Entity should have the const shared fragment", EntitySharedFragment);
		AITEST_EQUAL("Const shared fragment should hold the configured value", EntitySharedFragment->Value, TestIntValue);

		// Use the typed template overload (the new API being tested)
		AITEST_TRUE("Typed RemoveConstSharedFragmentFromEntity<T>() should succeed",
			EntityManager->RemoveConstSharedFragmentFromEntity<FTestConstSharedFragment_Int>(EntityHandle));

		EntitySharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
		AITEST_NULL("Entity should no longer have the const shared fragment after typed removal", EntitySharedFragment);

		AITEST_EQUAL("Entity's archetype should be back to the initial one", EntityManager->GetArchetypeForEntity(EntityHandle), FloatsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FConstSharedFragment_RemoveFromEntity_Typed, "System.Mass.ConstSharedFragments.RemoveFromEntity.Typed");

struct FSharedFragment_BatchAddToEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 1023;

		const FMassArchetypeHandle InitialArchetype = FloatsArchetype;
		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(InitialArchetype).GetNumEntitiesPerChunk();
		const int32 EntitiesToCreateNum = FMath::FloorToInt(float(EntitiesPerChunk) * 2.2f);
		const int32 EntitiesToMoveNum = FMath::FloorToInt(float(EntitiesPerChunk) * 1.2f);

		TArray<FMassEntityHandle> CreatedEntityHandles;

		EntityManager->BatchCreateEntities(InitialArchetype, EntitiesToCreateNum, CreatedEntityHandles);

		TArray<FMassEntityHandle> EntitiesToMove = CreatedEntityHandles;
		Algo::RandomShuffle(EntitiesToMove);
		TConstArrayView<FMassEntityHandle> EntitiesMoved = MakeArrayView(EntitiesToMove.GetData(), EntitiesToMoveNum);
		FMassArchetypeEntityCollection EntityCollection(InitialArchetype, EntitiesMoved, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);

		FMassArchetypeSharedFragmentValues SharedValues;
		FConstSharedStruct ConstSharedFragment = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue);
		SharedValues.Add(ConstSharedFragment);
		EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&EntityCollection, 1), SharedValues);

		const FMassArchetypeHandle TargetArchetype = EntityManager->GetArchetypeForEntityUnsafe(EntitiesToMove[0]);
		const int32 EntitiesMovedNum = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(TargetArchetype).GetNumEntities();
		AITEST_EQUAL("Number of entities moves needs to match expectations", EntitiesMovedNum, EntitiesToMoveNum);
		for (const FMassEntityHandle& EntityHandle : EntitiesMoved)
		{
			FTestConstSharedFragment_Int* SharedFragmentInstance = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityHandle);
			AITEST_NOT_NULL("Every entity moved needs to have a valid shared fragment", SharedFragmentInstance);
			AITEST_EQUAL("The shared fragment's value needs to match expectations", SharedFragmentInstance->Value, TestIntValue);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_BatchAddToEntity, "System.Mass.SharedFragments.BatchAddToEntity");

struct FSharedFragment_BatchSetAttempt : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 1023;
		constexpr int32 OtherTestIntValue = TestIntValue + 1;

		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();
		const int32 EntitiesToCreateNum = FMath::FloorToInt(float(EntitiesPerChunk) * 2.2f);
		const int32 EntitiesToMoveNum = FMath::FloorToInt(float(EntitiesPerChunk) * 1.2f);

		TArray<FMassEntityHandle> CreatedEntityHandles;
		FMassArchetypeSharedFragmentValues SharedIntValues;
		FConstSharedStruct ConstSharedFragment = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue);
		SharedIntValues.Add(ConstSharedFragment);

		TSharedRef<UE::Mass::ObserverManager::FCreationContext> CreationContext = EntityManager->BatchCreateEntities(FloatsArchetype, SharedIntValues, EntitiesToCreateNum, CreatedEntityHandles);
		const FMassArchetypeHandle ResultingArchetype = CreationContext->GetEntityCollections(*EntityManager.Get())[0].GetArchetype();

		FMassArchetypeEntityCollection EntityCollection(ResultingArchetype, CreatedEntityHandles, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
		
		// attempting to add the same values again should fail with checks and ensures
		{	
			MASS_SCOPED_ENSURE_TEST("Setting shared fragment values without archetype change is not supported", 1);
			MASS_SCOPED_ENSURE_TEST("Trying to set shared fragment values, without adding new shared fragments", 1);
			EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&EntityCollection, 1), SharedIntValues);
		}
		{
			FMassArchetypeSharedFragmentValues DifferentSharedIntValues;
			FConstSharedStruct OtherConstSharedFragment = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(OtherTestIntValue);
			DifferentSharedIntValues.Add(OtherConstSharedFragment);

			MASS_SCOPED_ENSURE_TEST("Setting shared fragment values without archetype change is not supported", 1);
			MASS_SCOPED_ENSURE_TEST("Trying to set shared fragment values, without adding new shared fragments", 1);
			EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&EntityCollection, 1), DifferentSharedIntValues);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_BatchSetAttempt, "System.Mass.SharedFragments.BatchSetAttempt");

struct FSharedFragment_BatchAddToEmpty : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 NumToReserve = 32;

		FMassArchetypeSharedFragmentValues SharedIntValues;
		FSharedStruct SharedFragment = FSharedStruct::Make<FTestSharedFragment_Int>();
		SharedIntValues.Add(SharedFragment);

		TArray<FMassEntityHandle> ReservedEntityHandles;
		EntityManager->BatchReserveEntities(NumToReserve, ReservedEntityHandles);
		
		FMassArchetypeEntityCollection EntityCollection(FMassArchetypeHandle(), ReservedEntityHandles, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates);
		// attempting to add the values before the entities are created is not a valid operation
		{
			MASS_SCOPED_ENSURE_TEST("Adding shared fragments to archetype-less entities is not supported", 1);
			EntityManager->BatchAddSharedFragmentsForEntities(MakeArrayView(&EntityCollection, 1), SharedIntValues);
		}
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragment_BatchAddToEmpty, "System.Mass.SharedFragments.BatchAddToEmpty");

struct FSharedFragmentValues_TypeEquivalency : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 TestIntValue = 32;

		FMassArchetypeSharedFragmentValues Values;
		const FMassSharedFragmentBitSet EmptySharedFragmentBitSet;
		const FMassConstSharedFragmentBitSet EmptyConstSharedFragmentBitSet;

		AITEST_TRUE("Empty shared values match type with empty bitset", HasExactSharedFragmentTypesMatch(Values, EmptySharedFragmentBitSet));
		AITEST_TRUE("Empty const shared values match type with empty const bitset", HasExactConstSharedFragmentTypesMatch(Values, EmptyConstSharedFragmentBitSet));

		const FMassSharedFragmentBitSet IntSharedFragmentBitSet = FMassSharedFragmentBitSet::GetTypeBitSet<FTestSharedFragment_Int>();
		FMassSharedFragmentBitSet IntFloatSharedFragmentBitSet = IntSharedFragmentBitSet;
		IntFloatSharedFragmentBitSet.Add<FTestSharedFragment_Float>();
		Values.Add(FSharedStruct::Make<FTestSharedFragment_Int>(TestIntValue));
		AITEST_TRUE("Single shared value type matches expected bitset", HasExactSharedFragmentTypesMatch(Values, IntSharedFragmentBitSet));
		AITEST_FALSE("Single shared value type doesn't match two-type bitset", HasExactSharedFragmentTypesMatch(Values, IntFloatSharedFragmentBitSet));
		AITEST_FALSE("Single shared value type doesn't match empty", HasExactSharedFragmentTypesMatch(Values, EmptySharedFragmentBitSet));

		const FMassConstSharedFragmentBitSet IntConstSharedFragmentBitSet = FMassConstSharedFragmentBitSet::GetTypeBitSet<FTestConstSharedFragment_Int>();
		FMassConstSharedFragmentBitSet IntFloatConstSharedFragmentBitSet = IntConstSharedFragmentBitSet;
		IntFloatConstSharedFragmentBitSet.Add<FTestConstSharedFragment_Float>();
		Values.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(TestIntValue));
		AITEST_TRUE("Single const shared value type matches expected bitset", HasExactConstSharedFragmentTypesMatch(Values, IntConstSharedFragmentBitSet));
		AITEST_FALSE("Single const shared value type doesn't match two-type bitset", HasExactConstSharedFragmentTypesMatch(Values, IntFloatConstSharedFragmentBitSet));
		AITEST_FALSE("Single const shared value type doesn't match empty", HasExactConstSharedFragmentTypesMatch(Values, EmptyConstSharedFragmentBitSet));

		Values.Remove(IntSharedFragmentBitSet);
		AITEST_TRUE("Emptied shared values match type with empty bitset", HasExactSharedFragmentTypesMatch(Values, EmptySharedFragmentBitSet));
		Values.Remove(IntConstSharedFragmentBitSet);
		AITEST_TRUE("Emptied const shared values match type with empty const bitset", HasExactConstSharedFragmentTypesMatch(Values, EmptyConstSharedFragmentBitSet));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_TypeEquivalency, "System.Mass.SharedFragments.TypeEquivalency");

struct FSharedFragmentValues_GetOrCreateWithArgs : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ConstIntValueOne = 1;
		constexpr int32 ConstIntValueTwo = 2;

		const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(/*Args*/ConstIntValueOne);
		const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(/*Args*/ConstIntValueOne);
		const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>(/*Args*/ConstIntValueTwo);

		AITEST_EQUAL("Shared fragments created for same struct type using same constructor value should share memory", SharedFragment1, SharedFragment2);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment", SharedFragment1.Get<FTestSharedFragment_Int>().Value, ConstIntValueOne);

		AITEST_NOT_EQUAL("Shared fragments created for same struct type using different constructor values should not share memory", SharedFragment1, SharedFragment3);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment", SharedFragment3.Get<FTestSharedFragment_Int>().Value, ConstIntValueTwo);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_GetOrCreateWithArgs, "System.Mass.SharedFragments.GetOrCreate.WithArgs");

struct FSharedFragmentValues_GetOrCreateWithStruct : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ConstIntValueOne = 1;
		constexpr int32 ConstIntValueTwo = 2;

		const FTestSharedFragment_Int TestSharedFragment_Int1(ConstIntValueOne);
		const FTestSharedFragment_Int TestSharedFragment_Int2(ConstIntValueOne);
		const FTestSharedFragment_Int TestSharedFragment_Int3(ConstIntValueTwo);
		const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment(TestSharedFragment_Int1);
		const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment(TestSharedFragment_Int2);
		const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment(TestSharedFragment_Int3);

		AITEST_EQUAL("Shared fragments created for same struct type using same constructor value should share memory", SharedFragment1, SharedFragment2);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment", SharedFragment1.Get<FTestSharedFragment_Int>().Value, ConstIntValueOne);

		AITEST_NOT_EQUAL("Shared fragments created for same struct type using different constructor values should not share memory", SharedFragment1, SharedFragment3);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateSharedFragment", SharedFragment3.Get<FTestSharedFragment_Int>().Value, ConstIntValueTwo);


		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_GetOrCreateWithStruct, "System.Mass.SharedFragments.GetOrCreate.WithStruct");

struct FSharedFragmentValues_GetOrCreateNoArgs : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>();
		const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>();
		const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Int>();

		AITEST_EQUAL("Shared fragments created for same struct type using default constructor should share memory", SharedFragment1, SharedFragment2);
		AITEST_EQUAL("Shared fragments created for same struct type using default constructor should share memory", SharedFragment1, SharedFragment3);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_GetOrCreateNoArgs, "System.Mass.SharedFragments.GetOrCreate.NoArgs");

struct FSharedFragmentValues_GetOrCreateArray : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FSharedStruct SharedFragment1 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Array>(TArray<int32>{ 1, 2, 3 });
		const FSharedStruct SharedFragment2 = EntityManager->GetOrCreateSharedFragment<FTestSharedFragment_Array>(TArray<int32>{ 1, 2, 3 });
		FTestSharedFragment_Array TestFragment;
		TestFragment.Value = { 1,2,3 };
		const FSharedStruct SharedFragment3 = EntityManager->GetOrCreateSharedFragment(TestFragment);

		AITEST_EQUAL("Shared fragments created for same struct type using same TArray contents should share memory", SharedFragment1, SharedFragment2);
		AITEST_EQUAL("Shared fragments created for same struct type using same TArray contents should share memory", SharedFragment1, SharedFragment3);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_GetOrCreateArray, "System.Mass.SharedFragments.GetOrCreate.WithArray");

struct FSharedFragmentValues_GetOrCreateConstNoArgs : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FConstSharedStruct SharedFragment1 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>();
		const FConstSharedStruct SharedFragment2 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>();
		const FConstSharedStruct SharedFragment3 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>();

		AITEST_EQUAL("Shared fragments created for same struct type using default constructor should share memory", SharedFragment1, SharedFragment2);
		AITEST_EQUAL("Shared fragments created for same struct type using default constructor should share memory", SharedFragment1, SharedFragment3);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_GetOrCreateConstNoArgs, "System.Mass.SharedFragments.GetOrCreate.ConstNoArgs");

struct FSharedFragmentValues_GetOrCreateConstWithArgs : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ConstIntValueOne = 1;
		constexpr int32 ConstIntValueTwo = 2;

		const FConstSharedStruct SharedFragment1 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(/*Args*/ConstIntValueOne);
		const FConstSharedStruct SharedFragment2 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(/*Args*/ConstIntValueOne);
		const FConstSharedStruct SharedFragment3 = EntityManager->GetOrCreateConstSharedFragment<FTestConstSharedFragment_Int>(/*Args*/ConstIntValueTwo);

		AITEST_EQUAL("Shared fragments created for same struct type using same constructor value should share memory", SharedFragment1, SharedFragment2);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment", SharedFragment1.Get<const FTestConstSharedFragment_Int>().Value, ConstIntValueOne);

		AITEST_NOT_EQUAL("Shared fragments created for same struct type using different constructor values should not share memory", SharedFragment1, SharedFragment3);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment", SharedFragment3.Get<const FTestConstSharedFragment_Int>().Value, ConstIntValueTwo);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_GetOrCreateConstWithArgs, "System.Mass.SharedFragments.GetOrCreate.ConstWithArgs");

struct FSharedFragmentValues_GetOrCreateConstWithStruct : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ConstIntValueOne = 1;
		constexpr int32 ConstIntValueTwo = 2;

		const FTestConstSharedFragment_Int TestSharedFragment_Int1(ConstIntValueOne);
		const FTestConstSharedFragment_Int TestSharedFragment_Int2(ConstIntValueOne);
		const FTestConstSharedFragment_Int TestSharedFragment_Int3(ConstIntValueTwo);
		const FConstSharedStruct SharedFragment1 = EntityManager->GetOrCreateConstSharedFragment(TestSharedFragment_Int1);
		const FConstSharedStruct SharedFragment2 = EntityManager->GetOrCreateConstSharedFragment(TestSharedFragment_Int2);
		const FConstSharedStruct SharedFragment3 = EntityManager->GetOrCreateConstSharedFragment(TestSharedFragment_Int3);

		AITEST_EQUAL("Shared fragments created for same struct type using same constructor value should share memory", SharedFragment1, SharedFragment2);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment", SharedFragment1.Get<const FTestConstSharedFragment_Int>().Value, ConstIntValueOne);

		AITEST_NOT_EQUAL("Shared fragments created for same struct type using different constructor values should not share memory", SharedFragment1, SharedFragment3);
		AITEST_EQUAL("Value in shared struct should be the same as the argument provided to GetOrCreateConstSharedFragment", SharedFragment3.Get<const FTestConstSharedFragment_Int>().Value, ConstIntValueTwo);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSharedFragmentValues_GetOrCreateConstWithStruct, "System.Mass.SharedFragments.GetOrCreate.ConstWithStruct");

} // FMassEntityTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
