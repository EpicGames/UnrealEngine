// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassArchetypeData.h"
#include "UObject/Class.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "Mass/EntityMacros.h"
#include "MassEntityTestTypes.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::SparseElements
{
#if WITH_MASSENTITY_DEBUG
	TNotNull<UScriptStruct*> MakeElement(FName TypeName, TNotNull<UScriptStruct*> BaseElementType)
	{
		UScriptStruct* NewScriptStruct = NewObject<UScriptStruct>(UScriptStruct::StaticClass(), FName(*FString::Printf(TEXT("%s"), *TypeName.ToString())), RF_Public);
		NewScriptStruct->SetSuperStruct(BaseElementType);
		NewScriptStruct->Bind();
		NewScriptStruct->PrepareCppStructOps();
		NewScriptStruct->StaticLink(true);
		NewScriptStruct->AddToRoot();
		return NewScriptStruct;
	}

	struct FAutoCleanupOperations
	{
		~FAutoCleanupOperations()
		{
			for (UScriptStruct* Type : CreatedTypes)
			{
				Type->RemoveFromRoot();
			}
		}

		TArray<UScriptStruct*> CreatedTypes;
	};

	struct FTagTestOperations : FAutoCleanupOperations
	{
		static bool HasSparseElement(const FMassArchetypeHandle& ArchetypeHandle, TNotNull<UScriptStruct*> SparseElementType)
		{
			return FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle).DoesContainEntitiesWithSparseElement(SparseElementType);
		}

		static bool HasSparseElement(FMassEntityView EntityView, TNotNull<UScriptStruct*> SparseElementType)
		{
			return EntityView.HasElement(SparseElementType);
		}

		TNotNull<UScriptStruct*> MakeSparseElement()
		{
			CreatedTypes.Add(MakeElement("SparseTagTestA", FMassSparseTag::StaticStruct()));
			return CreatedTypes.Last();
		}

		TNotNull<UScriptStruct*> MakeRegularElement()
		{
			CreatedTypes.Add(MakeElement("RegularTagTestA", FMassTag::StaticStruct()));
			return CreatedTypes.Last();
		}
	};

	struct FFragmentTestOperations : FAutoCleanupOperations
	{
		static bool HasSparseElement(const FMassArchetypeHandle& ArchetypeHandle, TNotNull<UScriptStruct*> SparseElementType)
		{
			return FMassArchetypeHelper::ArchetypeDataFromHandleChecked(ArchetypeHandle).DoesContainEntitiesWithSparseElement(SparseElementType);
		}

		static bool HasSparseElement(FMassEntityView EntityView, TNotNull<UScriptStruct*> SparseElementType)
		{
			return EntityView.HasElement(SparseElementType);
		}

		TNotNull<UScriptStruct*> MakeSparseElement()
		{
			CreatedTypes.Add(MakeElement("SparseFragmentTestA", FMassSparseFragment::StaticStruct()));
			return CreatedTypes.Last();
		}

		TNotNull<UScriptStruct*> MakeRegularElement()
		{
			CreatedTypes.Add(MakeElement("RegularFragmentTestA", FMassFragment::StaticStruct()));
			return CreatedTypes.Last();
		}
	};

	template<typename TTestOperations>
	struct FSparseElementsBase : FEntityTestBase
	{
		TTestOperations TestOperations;
		UScriptStruct* SparseElementType = nullptr;
		FMassArchetypeEntityCollection InitialCollection;
		TArray<FMassEntityHandle> EntitiesCreated;
		FMassArchetypeHandle OriginalArchetypeHandle;
		using FEntityTestBase::GetTestRunner;

		virtual bool SetUp() override
		{
			FEntityTestBase::SetUp();
			SparseElementType = TestOperations.MakeSparseElement();
			OriginalArchetypeHandle = FEntityTestBase::IntsArchetype;
			return true;
		}
	};

#define USING_SUPER(SuperType) \
		using Super = SuperType; \
		using Super::EntityManager; \
		using Super::OriginalArchetypeHandle; \
		using Super::EntitiesCreated; \
		using Super::InitialCollection; \
		using Super::SparseElementType; \
		using FEntityTestBase::GetTestRunner

	template<typename TTestOperations>
	struct FSparseElements_Add : FSparseElementsBase<TTestOperations>
	{
		USING_SUPER(FSparseElementsBase<TTestOperations>);

		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 2;
			constexpr int32 ModifiedEntityHandleIndex = 1;

			EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

			const FMassEntityHandle ModifiedEntityHandle = EntitiesCreated[ModifiedEntityHandleIndex];
			FStructView _ = EntityManager->AddSparseElementToEntity(ModifiedEntityHandle, SparseElementType);;

			const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
			AITEST_TRUE(TEXT("The affected entity is still in the original archetype"), NewArchetypeHandle == OriginalArchetypeHandle);
			AITEST_TRUE(TEXT("The archetype has the element")
				, TTestOperations::HasSparseElement(NewArchetypeHandle, SparseElementType));

			FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
			AITEST_TRUE(TEXT("The affected entity has the sparse element"), TTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));

			FMassEntityView OriginalEntityView(*EntityManager, EntitiesCreated[(ModifiedEntityHandleIndex + 1) % NumEntities]);
			AITEST_FALSE(TEXT("(NOT) The original entity has the sparse element"), TTestOperations::HasSparseElement(OriginalEntityView, SparseElementType));

			return true;
		}
	};
	using FSparseElements_AddTag = FSparseElements_Add<FTagTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_AddTag, "System.Mass.Sparse.Tag.Add");
	using FSparseElements_AddFragment = FSparseElements_Add<FFragmentTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_AddFragment, "System.Mass.Sparse.Fragment.Add");

	template<typename TTestOperations>
	struct FSparseElements_Remove : FSparseElementsBase<TTestOperations>
	{
		USING_SUPER(FSparseElementsBase<TTestOperations>);

		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 2;
			
			EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

			for (FMassEntityHandle EntityHandle : EntitiesCreated)
			{
				FStructView _ = EntityManager->AddSparseElementToEntity(EntityHandle, SparseElementType);;
			}

			EntityManager->RemoveSparseElementFromEntity(EntitiesCreated[0], SparseElementType);

			const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(EntitiesCreated[0]);
			AITEST_TRUE(TEXT("The affected entity is still in the original archetype"), NewArchetypeHandle == OriginalArchetypeHandle);
			AITEST_TRUE(TEXT("The archetype knows some of its entities have the sparse element"), FMassArchetypeHelper::ArchetypeDataFromHandleChecked(NewArchetypeHandle).ContainsAnySparseData());

			FMassEntityView ModifiedEntityView(*EntityManager, EntitiesCreated[0]);
			AITEST_FALSE(TEXT("(NOT) The affected entity has the sparse element"), TTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));

			FMassEntityView OtherEntityView(*EntityManager, EntitiesCreated[1]);
			AITEST_TRUE(TEXT("The other entity has the sparse element"), TTestOperations::HasSparseElement(OtherEntityView, SparseElementType));

			EntityManager->RemoveSparseElementFromEntity(EntitiesCreated[1], SparseElementType);
			AITEST_FALSE(TEXT("(NOT) The other entity has the sparse element"), TTestOperations::HasSparseElement(OtherEntityView, SparseElementType));

			AITEST_TRUE(TEXT("The original archetype no longer has the element")
				, TTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

			return true;
		}
	};
	using FSparseElements_RemoveTag = FSparseElements_Remove<FTagTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_RemoveTag, "System.Mass.Sparse.Tag.Remove");
	using FSparseElements_RemoveFragment = FSparseElements_Remove<FFragmentTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_RemoveFragment, "System.Mass.Sparse.Fragment.Remove");

	template<typename TTestOperations>
	struct FSparseElements_MoveEntity : FSparseElementsBase<TTestOperations>
	{
		USING_SUPER(FSparseElementsBase<TTestOperations>);

		virtual bool InstantTest() override
		{
			const FMassEntityHandle ModifiedEntityHandle = EntityManager->CreateEntity(OriginalArchetypeHandle);

			FStructView _ = EntityManager->AddSparseElementToEntity(ModifiedEntityHandle, SparseElementType);;

			EntityManager->AddFragmentToEntity(ModifiedEntityHandle, FTestFragment_Float::StaticStruct());

			AITEST_TRUE(TEXT("The original archetype no longer has the element")
				, TTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

			const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
			AITEST_TRUE(TEXT("The entity is expected to end up in the expected archetype"), NewArchetypeHandle == FEntityTestBase::FloatsIntsArchetype);

			FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
			AITEST_TRUE(TEXT("The affected entity has the sparse element"), TTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
			
			return true;
		}
	};
	using FSparseElements_MoveEntityTag = FSparseElements_MoveEntity<FTagTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_MoveEntityTag, "System.Mass.Sparse.Tag.Move");
	using FSparseElements_MoveEntityFragment = FSparseElements_MoveEntity<FFragmentTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_MoveEntityFragment, "System.Mass.Sparse.Fragment.Move");

	// @todo add batch-add and batch-remove
	template<typename TTestOperations>
	struct FSparseElements_BatchMoveEntities : FSparseElementsBase<TTestOperations>
	{
		USING_SUPER(FSparseElementsBase<TTestOperations>);

		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 2;
			
			EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

			for (FMassEntityHandle EntityHandle : EntitiesCreated)
			{
				FStructView _ = EntityManager->AddSparseElementToEntity(EntityHandle, SparseElementType);;
			}

			InitialCollection = FMassArchetypeEntityCollection(FEntityTestBase::IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchChangeFragmentCompositionForEntities({InitialCollection}, FMassFragmentBitSet(FTestFragment_Float::StaticStruct()), {});

			AITEST_TRUE(TEXT("The original archetype no longer has the element")
				, TTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

			FMassEntityHandle ModifiedEntityHandle = EntitiesCreated[0];
			const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
			AITEST_TRUE(TEXT("The entity is expected to end up in the expected archetype"), NewArchetypeHandle == FEntityTestBase::FloatsIntsArchetype);

			FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
			AITEST_TRUE(TEXT("The affected entity has the sparse element"), TTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
			
			return true;
		}
	};
	using FSparseElements_BatchMoveEntitiesTag = FSparseElements_BatchMoveEntities<FTagTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchMoveEntitiesTag, "System.Mass.Sparse.Tag.BatchMove");
	using FSparseElements_BatchMoveEntitiesFragment = FSparseElements_BatchMoveEntities<FFragmentTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchMoveEntitiesFragment, "System.Mass.Sparse.Fragment.BatchMove");

	template<typename TTestOperations>
	struct FSparseElements_BatchAdd : FSparseElementsBase<TTestOperations>
	{
		USING_SUPER(FSparseElementsBase<TTestOperations>);

		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 2;

			EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

			InitialCollection = FMassArchetypeEntityCollection(FEntityTestBase::IntsArchetype, EntitiesCreated, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

			AITEST_TRUE(TEXT("The original archetype has the sparse element")
				, TTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType));

			for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
			{
				const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
				AITEST_TRUE(TEXT("The entity is expected to end up in the expected archetype"), NewArchetypeHandle == OriginalArchetypeHandle);

				FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
				AITEST_TRUE(TEXT("The affected entities have the sparse element"), TTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
			}
			
			return true;
		}
	};
	using FSparseElements_BatchAddTag = FSparseElements_BatchAdd<FTagTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchAddTag, "System.Mass.Sparse.Tag.BatchAdd");
	using FSparseElements_BatchAddFragment = FSparseElements_BatchAdd<FFragmentTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchAddFragment, "System.Mass.Sparse.Fragment.BatchAdd");

	template<typename TTestOperations>
	struct FSparseElements_BatchRemove : FSparseElements_BatchAdd<TTestOperations>
	{
		USING_SUPER(FSparseElements_BatchAdd<TTestOperations>);

		virtual bool InstantTest() override
		{
			if (Super::InstantTest() == false)
			{
				return false;
			}

			EntityManager->BatchRemoveSparseElementFromEntities({InitialCollection}, SparseElementType);

			AITEST_TRUE(TEXT("The original archetype no longer has the element")
				, TTestOperations::HasSparseElement(OriginalArchetypeHandle, SparseElementType) == false);

			for (FMassEntityHandle ModifiedEntityHandle : EntitiesCreated)
			{
				const FMassArchetypeHandle NewArchetypeHandle = EntityManager->GetArchetypeForEntity(ModifiedEntityHandle);
				AITEST_TRUE(TEXT("The entity is expected to end up in the expected archetype"), NewArchetypeHandle == OriginalArchetypeHandle);

				FMassEntityView ModifiedEntityView(*EntityManager, ModifiedEntityHandle);
				AITEST_FALSE(TEXT("(NOT) The affected entities have the sparse element"), TTestOperations::HasSparseElement(ModifiedEntityView, SparseElementType));
			}
			
			return true;
		}
	};
	using FSparseElements_BatchRemoveTag = FSparseElements_BatchRemove<FTagTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchRemoveTag, "System.Mass.Sparse.Tag.BatchRemove");
	using FSparseElements_BatchRemoveFragment = FSparseElements_BatchRemove<FFragmentTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchRemoveFragment, "System.Mass.Sparse.Fragment.BatchRemove");

	template<typename TTestOperations>
	struct FSparseElements_Query : FSparseElementsBase<TTestOperations>
	{
		USING_SUPER(FSparseElementsBase<TTestOperations>);

		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 5;
			constexpr int32 NumEntitiesAffected = 3;
			EntityManager->BatchCreateEntities(OriginalArchetypeHandle, NumEntities, EntitiesCreated);

			TConstArrayView<FMassEntityHandle> ModifiedEntities = MakeArrayView(&EntitiesCreated[1], NumEntitiesAffected);
			InitialCollection = FMassArchetypeEntityCollection(FEntityTestBase::IntsArchetype, ModifiedEntities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({InitialCollection}, SparseElementType);

			// 1. Regular + Sparse
			{
				TArray<FMassEntityHandle> VerifyEntities1;

				FMassEntityQuery Query(EntityManager);
				Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
				Query.AddSparseRequirement(SparseElementType);

				FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
				FAutomationTestBase& LocalTestRunner = GetTestRunner();
				Query.ForEachEntityChunk(ExecutionContext, [SparseElementType=SparseElementType, &VerifyEntities1, &LocalTestRunner](FMassExecutionContext& Context)
				{
					// option 1: the archetype has the sparse element, and we need to check individual entities
					for (auto EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						if (const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntityIt))
						{
							if (UE::Mass::IsA<FMassFragment>(SparseElementType))
							{
								FConstStructView ElementView = Context.GetSparseElement(SparseElementType, EntityIt);
								AITEST_TRUE_EX(LocalTestRunner, TEXT("The filtered entity has the sparse element, const view"), ElementView.IsValid());

								FStructView ElementMutableView = Context.GetMutableSparseElement(SparseElementType, EntityIt);
								AITEST_TRUE_EX(LocalTestRunner, TEXT("The filtered entity has the sparse element, mutable view"), ElementMutableView.IsValid());

								AITEST_TRUE_EX(LocalTestRunner, TEXT("Both views point at the same element"), ElementView == ElementMutableView);
							}

							VerifyEntities1.Add(Context.GetEntities()[EntityIt]);
						}
					}
					return true;
				});

				AITEST_EQUAL("Number of expected entities, option 1", VerifyEntities1.Num(), NumEntitiesAffected);

				TArray<FMassEntityHandle> VerifyEntities2;
				Query.ForEachEntityChunk(ExecutionContext, [SparseElementType=SparseElementType, &VerifyEntities2, &LocalTestRunner](FMassExecutionContext& Context)
				{
					// option 2: dedicated sparse element iterator that skips entities that don't have the elements
					// CreateSparseEntityIterator:
					// * derives from FMassExecutionContext::FEntityIterator and overrides operator++ to skip elements not matching the sparse elements composition
					// * compares against sparse-elements requirements expressed by the query (as stored in the Context).
					for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
					{
						// all the following would be expected to succeed.
						const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntitySparseIt);
						AITEST_TRUE_EX(LocalTestRunner, TEXT("The filtered entity has the sparse element"), bHasSparseElement);

						if (UE::Mass::IsA<FMassFragment>(SparseElementType))
						{
							FConstStructView ElementView = Context.GetSparseElement(SparseElementType, EntitySparseIt);
							AITEST_TRUE_EX(LocalTestRunner, TEXT("The filtered entity has the sparse element, const view"), ElementView.IsValid());

							FStructView ElementMutableView = Context.GetMutableSparseElement(SparseElementType, EntitySparseIt);
							AITEST_TRUE_EX(LocalTestRunner, TEXT("The filtered entity has the sparse element, mutable view"), ElementMutableView.IsValid());

							AITEST_TRUE_EX(LocalTestRunner, TEXT("Both views point at the same element"), ElementView == ElementMutableView);
						}

						VerifyEntities2.Add(Context.GetEntities()[EntitySparseIt]);
					}

					return true;
				});

				AITEST_EQUAL("Number of expected entities, option 2", VerifyEntities2.Num(), NumEntitiesAffected);
			}

			// 2. No-sparse
			{
				FMassEntityQuery Query(EntityManager);
				Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly/*, EMassFragmentAccess::ReadOnly*/);
				Query.AddSparseRequirement(SparseElementType, EMassFragmentPresence::None);

				FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
				FAutomationTestBase& LocalTestRunner = GetTestRunner();
				
				TArray<FMassEntityHandle> VerifyEntities;
				Query.ForEachEntityChunk(ExecutionContext, [SparseElementType=SparseElementType, &VerifyEntities, &LocalTestRunner](FMassExecutionContext& Context)
				{
					for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
					{
						const bool bHasSparseElement = Context.HasSparseElement(SparseElementType, EntitySparseIt);
						AITEST_FALSE_EX(LocalTestRunner, TEXT("(NOT) The filtered entity has the sparse element"), bHasSparseElement);
						VerifyEntities.Add(Context.GetEntities()[EntitySparseIt]);
					}

					return true;
				});

				AITEST_EQUAL("Number of expected entities, no-sparse", VerifyEntities.Num(), (NumEntities - NumEntitiesAffected));
			}

			return true;
		}
	};
	using FSparseElements_QueryTag = FSparseElements_Query<FTagTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_QueryTag, "System.Mass.Sparse.Query.Tag");
	using FSparseElements_QueryFragment = FSparseElements_Query<FFragmentTestOperations>;
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_QueryFragment, "System.Mass.Sparse.Query.Fragment");


	struct FSparseElements_QueryFragmentModification : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 5;
			constexpr int32 NumEntitiesAffected = 3;

			TArray<FMassEntityHandle> EntitiesCreated;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, EntitiesCreated);

			TConstArrayView<FMassEntityHandle> ModifiedEntities = MakeArrayView(&EntitiesCreated[1], NumEntitiesAffected);
			FMassArchetypeEntityCollection InitialCollection = FMassArchetypeEntityCollection(FEntityTestBase::IntsArchetype, ModifiedEntities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({InitialCollection}, FTestFragment_SparseInt::StaticStruct());

			FMassEntityQuery Query(EntityManager);
			Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			Query.AddSparseRequirement<FTestFragment_SparseInt>();

			FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
			FAutomationTestBase& LocalTestRunner = GetTestRunner();

			int32 Counter = 0;
			TArray<FMassEntityHandle> VerifyEntities;
			Query.ForEachEntityChunk(ExecutionContext, [&Counter, &VerifyEntities, &LocalTestRunner](FMassExecutionContext& Context)
			{
				for (FMassSparseEntityIterator EntitySparseIt = Context.CreateSparseEntityIterator(); EntitySparseIt; ++EntitySparseIt)
				{
					FTestFragment_SparseInt* FragmentInstance = Context.GetMutableSparseElement<FTestFragment_SparseInt>(EntitySparseIt);
					AITEST_NOT_NULL_EX(LocalTestRunner, TEXT("The filtered entity has the sparse element"), FragmentInstance);

					FragmentInstance->Value = Counter++;

					VerifyEntities.Add(Context.GetEntities()[EntitySparseIt]);
				}

				return true;
			});

			for (int32 EntityIndex = 0; EntityIndex < VerifyEntities.Num(); ++EntityIndex)
			{
				FMassEntityView EntityView(*EntityManager, VerifyEntities[EntityIndex]);
				FTestFragment_SparseInt* FragmentPtr = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				AITEST_NOT_NULL(TEXT("Collected entity's sparse fragment"), FragmentPtr);
				AITEST_EQUAL(TEXT("Sparse fragment's expected value"), FragmentPtr->Value, EntityIndex);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_QueryFragmentModification, "System.Mass.Sparse.Query.ModifyFragment");

	struct FSparseElements_QuerySkipChunks : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
			constexpr int32 NumTotalChunks = 5;
			const int32 NumEntities = NumTotalChunks * EntitiesPerChunk;
			constexpr int32 NumAffectedChunks = 2;
			const int32 NumEntitiesAffected = NumAffectedChunks * EntitiesPerChunk;

			TArray<FMassEntityHandle> EntitiesCreated;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, EntitiesCreated);

			// ModifiedEntities should contain all the entities in chunks 1 & 2
			TConstArrayView<FMassEntityHandle> ModifiedEntities = MakeArrayView(&EntitiesCreated[EntitiesPerChunk], NumEntitiesAffected);
			FMassArchetypeEntityCollection InitialCollection = FMassArchetypeEntityCollection(FEntityTestBase::IntsArchetype, ModifiedEntities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({InitialCollection}, FTestFragment_SparseInt::StaticStruct());

			FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
			FMassEntityQuery Query(EntityManager);
			Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			Query.AddSparseRequirement<FTestFragment_SparseInt>();

			int32 Counter = 0;
			Query.ForEachEntityChunk(ExecutionContext, [&Counter](FMassExecutionContext& Context)
			{
				++Counter;
			});
			AITEST_EQUAL(TEXT("Expected number of chunks processed"), Counter, NumAffectedChunks);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_QuerySkipChunks, "System.Mass.Sparse.Query.SkipChunks");

	struct FSparseElements_Storage_Individual : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			FSparseElementsStorage Storage;

			const FMassEntityHandle EntityHandle = EntityManager->CreateEntity(IntsArchetype);

			FConstStructView NoView = Storage.GetElementDataForEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
			AITEST_TRUE(TEXT("Initially entity doesn't have the sparse element"), NoView.IsValid() == false);

			FStructView ElementView = Storage.AddElementToEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
			AITEST_TRUE(TEXT("View of added element is valid"), ElementView.IsValid());

			FStructView ExistingView = Storage.GetMutableElementDataForEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
			AITEST_TRUE(TEXT("Existing fragment view is valid"), ExistingView.IsValid());
			AITEST_EQUAL(TEXT("Both views point to the same fragment"), ElementView, ExistingView);

			const bool bRemoved = Storage.RemoveElementFromEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
			AITEST_TRUE(TEXT("Removing the added element succeeds"), bRemoved);

			const bool bSecondRemove = Storage.RemoveElementFromEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
			AITEST_TRUE(TEXT("Removing the added element for the second time fails"), bSecondRemove == false);

			FConstStructView RemovedView = Storage.GetElementDataForEntity(EntityHandle, FTestFragment_SparseInt::StaticStruct());
			AITEST_TRUE(TEXT("Fetching the removed element results in a N invalid view"), RemovedView.IsValid() == false);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_Storage_Individual, "System.Mass.Sparse.Storage.IndividualBasic");

	struct FSparseElements_Storage_Iterator : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 100;
			int32 NumWithSparseElement = 0;

			FRandomStream RandStream(0);

			FSparseElementsStorage Storage;

			TArray<FMassEntityHandle> EntitiesCreated;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, EntitiesCreated);

			// add sparse elements
			TArray<FMassEntityHandle> ModifiedEntities;
			for (int32 EntityIndex = 0; EntityIndex < EntitiesCreated.Num(); ++EntityIndex)
			{
				if (RandStream.FRand() < 0.3f)
				{
					Storage.AddElementToEntity<FTestFragment_SparseInt>(EntitiesCreated[EntityIndex]);
					ModifiedEntities.Add(EntitiesCreated[EntityIndex]);
				}
			}

			int32 IteratedCount = 0;
			TSet<int32> IteratedEntityIndices;

			for (UE::Mass::FSparseElementIterator It = Storage.CreateElementIterator(FTestFragment_SparseInt::StaticStruct()); It; ++It)
			{
				++IteratedCount;

				FStructView ElementView = It.GetElementView();
				AITEST_TRUE(TEXT("Iterated element view is valid"), ElementView.IsValid());

				const int32 EntityIndex = It.GetEntityIndex();
				IteratedEntityIndices.Add(EntityIndex);

				// perform modification, we'll test it later on
				FTestFragment_SparseInt* Fragment = ElementView.GetPtr<FTestFragment_SparseInt>();
				Fragment->Value = EntityIndex;
			}

			AITEST_EQUAL(TEXT("Iterated over all modified entities"), IteratedCount, ModifiedEntities.Num());

			for (const FMassEntityHandle& Entity : ModifiedEntities)
			{
				AITEST_TRUE(TEXT("Iterated over modified entity"), IteratedEntityIndices.Contains(Entity.Index));
			}

			// test the values written
			for (UE::Mass::FSparseElementIterator It = Storage.CreateElementIterator(FTestFragment_SparseInt::StaticStruct()); It; ++It)
			{
				FConstStructView ElementView = It.GetElementView();
				AITEST_TRUE(TEXT("Const view is valid"), ElementView.IsValid());

				const int32 EntityIndex = It.GetEntityIndex();
				const FTestFragment_SparseInt* Fragment = ElementView.GetPtr<FTestFragment_SparseInt>();
				AITEST_EQUAL(TEXT("Stored value matches expectations"), Fragment->Value, EntityIndex);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_Storage_Iterator, "System.Mass.Sparse.Storage.Iterator");

#undef USING_SUPER
#endif // WITH_MASSENTITY_DEBUG
} // namespace

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
