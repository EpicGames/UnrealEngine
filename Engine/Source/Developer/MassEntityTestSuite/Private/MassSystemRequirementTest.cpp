// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "Engine/Engine.h"
#include "MassProcessorDependencySolver.h"
#include "MassEntityTestTypes.h"
#include "MassSubsystemAccess.h"
#include "MassExecutionContext.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//-----------------------------------------------------------------------------
// Tests
//----------------------------------------------------------------------------- 
namespace FMassSystemRequirementTest
{

struct FSystemRequirementTestBase : FEntityTestBase
{
	using Super = FEntityTestBase;
	FMassEntityQuery EntityQuery;

	FSystemRequirementTestBase()
	{
		bMakeWorldEntityManagersOwner = true;
	}

	virtual bool SetUp() override
	{
		if (Super::SetUp())
		{
			EntityManager->CreateEntity(FloatsArchetype);
			EntityQuery.Initialize(EntityManager.ToSharedRef());
			EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
			return true;
		}

		return false;
	}
};	

//-----------------------------------------------------------------------------
// The main point of this specific test is to verify that we're getting the same answers from both
// FMassEntityQuery and FMassSubsystemAccess.
//----------------------------------------------------------------------------- 
struct FGenericSubsystemAccessAPI : FSystemRequirementTestBase
{
	virtual bool InstantTest() override
	{
		UWorld* World = FAITestHelpers::GetWorld();
		check(World);
		FMassSubsystemAccess SubsystemAccessBits(EntityManager.Get());
		FMassSubsystemAccess SubsystemAccessClasses(EntityManager.Get());

		EntityQuery.AddSubsystemRequirement<UMassTestWorldSubsystem>(EMassFragmentAccess::ReadWrite);
		EntityQuery.AddSubsystemRequirement<UMassTestEngineSubsystem>(EMassFragmentAccess::ReadWrite);

		SubsystemAccessBits.SetSubsystemRequirementBits(EntityQuery.GetRequiredConstSubsystems(), EntityQuery.GetRequiredMutableSubsystems());
		SubsystemAccessClasses.SetSubsystemRequirements(EntityQuery);

		UMassTestWorldSubsystem* TestWorldSubsystemActual = World->GetSubsystem<UMassTestWorldSubsystem>();
		UMassTestEngineSubsystem* TestEngineSubsystemActual = GEngine->GetEngineSubsystem<UMassTestEngineSubsystem>();

		UMassTestWorldSubsystem* AccessWorldSubsystem = nullptr;
		UMassTestEngineSubsystem* AccessEngineSubsystem = nullptr;

		const UMassTestWorldSubsystem* ConstAccessWorldSubsystem = nullptr;
		const UMassTestEngineSubsystem* ConstAccessEngineSubsystem = nullptr;

		FMassExecutionContext ExecutionContext(*EntityManager);
		EntityQuery.ForEachEntityChunk(ExecutionContext, [&](FMassExecutionContext& Context)
			{
				AccessWorldSubsystem = Context.GetMutableSubsystem<UMassTestWorldSubsystem>();
				AccessEngineSubsystem = Context.GetMutableSubsystem<UMassTestEngineSubsystem>();

				ConstAccessWorldSubsystem = Context.GetSubsystem<UMassTestWorldSubsystem>();
				ConstAccessEngineSubsystem = Context.GetSubsystem<UMassTestEngineSubsystem>();
			});

		AITEST_NOT_NULL(TEXT("WorldSubsystem: Subsystem Actual is expected to be not NULL"), TestWorldSubsystemActual);
		AITEST_EQUAL(TEXT("WorldSubsystem: Mutable Subsystem fetched is expected to be the same as the Actual"), TestWorldSubsystemActual, AccessWorldSubsystem);
		AITEST_EQUAL(TEXT("WorldSubsystem: Const Subsystem fetched is expected to be the same as the Actual"), TestWorldSubsystemActual, ConstAccessWorldSubsystem);
		AITEST_EQUAL(TEXT("WorldSubsystem: Subsystem fetched via bits-requirements is expected to be the same as the Actual")
			, TestWorldSubsystemActual, SubsystemAccessBits.GetSubsystem<UMassTestWorldSubsystem>());
		AITEST_EQUAL(TEXT("WorldSubsystem: Subsystem fetched via class-requirements is expected to be the same as the Actual")
			, TestWorldSubsystemActual, SubsystemAccessClasses.GetSubsystem<UMassTestWorldSubsystem>());

		AITEST_NOT_NULL(TEXT("EngineSubsystem: Subsystem Actual is expected to be not NULL"), TestEngineSubsystemActual);
		AITEST_EQUAL(TEXT("EngineSubsystem: Mutable Subsystem fetched is expected to be the same as the Actual"), TestEngineSubsystemActual, AccessEngineSubsystem);
		AITEST_EQUAL(TEXT("EngineSubsystem: Const Subsystem fetched is expected to be the same as the Actual"), TestEngineSubsystemActual, ConstAccessEngineSubsystem);
		AITEST_EQUAL(TEXT("EngineSubsystem: Subsystem fetched via bits-requirements is expected to be the same as the Actual")
			, TestEngineSubsystemActual, SubsystemAccessBits.GetSubsystem<UMassTestEngineSubsystem>());
		AITEST_EQUAL(TEXT("EngineSubsystem: Subsystem fetched via class-requirements is expected to be the same as the Actual")
			, TestEngineSubsystemActual, SubsystemAccessClasses.GetSubsystem<UMassTestEngineSubsystem>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FGenericSubsystemAccessAPI, "System.Mass.Query.System.AccessAPI.Generic");

struct FGameSubsystemAccessAPI : FSystemRequirementTestBase
{
	virtual bool InstantTest() override
	{
		UWorld* World = FAITestHelpers::GetWorld();
		check(World);

		const ULocalPlayer* LocalPlayer = World->GetFirstLocalPlayerFromController();
		UMassTestLocalPlayerSubsystem* TestLocalPlayerSubsystemActual = LocalPlayer ? LocalPlayer->GetSubsystem<UMassTestLocalPlayerSubsystem>() : nullptr;
		const UGameInstance* GameInstance = World->GetGameInstance();
		UMassTestGameInstanceSubsystem* TestGameInstanceSubsystemActual = GameInstance ? GameInstance->GetSubsystem<UMassTestGameInstanceSubsystem>() : nullptr;

		if (World->IsGameWorld() == false)
		{
			// no point in running the full test since none of the used system classes should produce any instances
			// just verify this assumption
			AITEST_NULL(TEXT("LocalPlayerSubsystem subclass instance is expected to be NULL in non-game worlds"), TestLocalPlayerSubsystemActual);
			AITEST_NULL(TEXT("GameInstanceSubsystem subclass instance is expected to be NULL in non-game worlds"), TestGameInstanceSubsystemActual);

			return true;
		}

		FMassSubsystemAccess SubsystemAccessBits(EntityManager.Get());
		FMassSubsystemAccess SubsystemAccessClasses(EntityManager.Get());

		EntityQuery.AddSubsystemRequirement<UMassTestLocalPlayerSubsystem>(EMassFragmentAccess::ReadWrite);
		EntityQuery.AddSubsystemRequirement<UMassTestGameInstanceSubsystem>(EMassFragmentAccess::ReadWrite);

		SubsystemAccessBits.SetSubsystemRequirementBits(EntityQuery.GetRequiredConstSubsystems(), EntityQuery.GetRequiredMutableSubsystems());
		SubsystemAccessClasses.SetSubsystemRequirements(EntityQuery);

		UMassTestLocalPlayerSubsystem* AccessLocalPlayerSubsystem = nullptr;
		UMassTestGameInstanceSubsystem* AccessGameInstanceSubsystem = nullptr;

		const UMassTestLocalPlayerSubsystem* ConstAccessLocalPlayerSubsystem = nullptr;
		const UMassTestGameInstanceSubsystem* ConstAccessGameInstanceSubsystem = nullptr;

		FMassExecutionContext ExecutionContext(*EntityManager);
		EntityQuery.ForEachEntityChunk(ExecutionContext, [&](FMassExecutionContext& Context)
			{
				AccessLocalPlayerSubsystem = Context.GetMutableSubsystem<UMassTestLocalPlayerSubsystem>();
				AccessGameInstanceSubsystem = Context.GetMutableSubsystem<UMassTestGameInstanceSubsystem>();

				ConstAccessLocalPlayerSubsystem = Context.GetSubsystem<UMassTestLocalPlayerSubsystem>();
				ConstAccessGameInstanceSubsystem = Context.GetSubsystem<UMassTestGameInstanceSubsystem>();
			});

		AITEST_EQUAL(TEXT("LocalPlayerSubsystem: Mutable Subsystem fetched is expected to be the same as the Actual"), TestLocalPlayerSubsystemActual, AccessLocalPlayerSubsystem);
		AITEST_EQUAL(TEXT("LocalPlayerSubsystem: Const Subsystem fetched is expected to be the same as the Actual"), TestLocalPlayerSubsystemActual, ConstAccessLocalPlayerSubsystem);
		AITEST_EQUAL(TEXT("LocalPlayerSubsystem: Subsystem fetched via bits-requirements is expected to be the same as the Actual")
			, TestLocalPlayerSubsystemActual, SubsystemAccessBits.GetSubsystem<UMassTestLocalPlayerSubsystem>());
		AITEST_EQUAL(TEXT("LocalPlayerSubsystem: Subsystem fetched via class-requirements is expected to be the same as the Actual")
			, TestLocalPlayerSubsystemActual, SubsystemAccessClasses.GetSubsystem<UMassTestLocalPlayerSubsystem>());

		AITEST_EQUAL(TEXT("GameInstanceSubsystem: Mutable Subsystem fetched is expected to be the same as the Actual"), TestGameInstanceSubsystemActual, AccessGameInstanceSubsystem);
		AITEST_EQUAL(TEXT("GameInstanceSubsystem: Const Subsystem fetched is expected to be the same as the Actual"), TestGameInstanceSubsystemActual, ConstAccessGameInstanceSubsystem);
		AITEST_EQUAL(TEXT("GameInstanceSubsystem: Subsystem fetched via bits-requirements is expected to be the same as the Actual")
			, TestGameInstanceSubsystemActual, SubsystemAccessBits.GetSubsystem<UMassTestGameInstanceSubsystem>());
		AITEST_EQUAL(TEXT("GameInstanceSubsystem: Subsystem fetched via class-requirements is expected to be the same as the Actual")
			, TestGameInstanceSubsystemActual, SubsystemAccessClasses.GetSubsystem<UMassTestGameInstanceSubsystem>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FGameSubsystemAccessAPI, "System.Mass.Query.System.AccessAPI.Game");

} // FMassSystemRequirementTest

#undef LOCTEXT_NAMESPACE

UE_ENABLE_OPTIMIZATION_SHIP