// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTestTypes.h"
#include "MassEntityManager.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassProcessingContext.h"
#include "Engine/World.h"
#include "MassProcessingContext.h"


UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// Test bases 
//----------------------------------------------------------------------//
bool FExecutionTestBase::SetUp()
{
	EntityManager = MakeShareable(new FMassEntityManager(bMakeWorldEntityManagersOwner ? FAITestHelpers::GetWorld() : nullptr));
	EntityManager->SetDebugName(TEXT("MassEntityTestSuite"));

	FMassEntityManagerStorageInitParams InitializationParams;
#if WITH_MASS_CONCURRENT_RESERVE

	InitializationParams.Emplace<FMassEntityManager_InitParams_Concurrent>();
#else
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InitializationParams.Emplace<FMassEntityManager_InitParams_SingleThreaded>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_MASS_CONCURRENT_RESERVE

	EntityManager->Initialize(InitializationParams);
	EntityManager->PostInitialize();

	return true;
}

void FExecutionTestBase::TearDown()
{
	FAITestBase::TearDown();

	EntityManager->Deinitialize();
	EntityManager = nullptr;
}

bool FEntityTestBase::SetUp()
{
	FExecutionTestBase::SetUp();
	check(EntityManager);

	const UScriptStruct* FragmentTypes[] = { FTestFragment_Float::StaticStruct(), FTestFragment_Int::StaticStruct(), FTestFragment_Large::StaticStruct()};

	EmptyArchetype = EntityManager->CreateArchetype(MakeArrayView<const UScriptStruct*>(nullptr, 0));
	FloatsArchetype = EntityManager->CreateArchetype(MakeArrayView(&FragmentTypes[0], 1));
	IntsArchetype = EntityManager->CreateArchetype(MakeArrayView(&FragmentTypes[1], 1));
	FloatsIntsArchetype = EntityManager->CreateArchetype(MakeArrayView(FragmentTypes, 2));
	LargeArchetype = EntityManager->CreateArchetype(MakeArrayView(&FragmentTypes[2], 1));

	FTestFragment_Int IntFrag;
	IntFrag.Value = FTestFragment_Int::TestIntValue;
	InstanceInt = FInstancedStruct::Make(IntFrag);

	return true;
}


//----------------------------------------------------------------------//
// Processors 
//----------------------------------------------------------------------//
UMassTestProcessorBase::UMassTestProcessorBase()
	: EntityQuery(*this)
{
#if WITH_EDITORONLY_DATA
	bCanShowUpInSettings = false;
#endif // WITH_EDITORONLY_DATA
	bAutoRegisterWithProcessingPhases = false;
	ExecutionFlags = int32(EProcessorExecutionFlags::All);

	ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context) {};

	SetUseParallelForEachEntityChunk(false);
}

void UMassTestProcessorBase::SetUseParallelForEachEntityChunk(bool bEnable)
{
	if (bEnable)
	{
		ExecutionFunction = [this](FMassEntityManager& InEntityManager, FMassExecutionContext& Context) 
		{
			EntityQuery.ParallelForEachEntityChunk(Context, ForEachEntityChunkExecutionFunction, FMassEntityQuery::EParallelExecutionFlags::Force);
		};
	}
	else
	{
		ExecutionFunction = [this](FMassEntityManager& InEntityManager, FMassExecutionContext& Context) 
		{
			EntityQuery.ForEachEntityChunk(Context, ForEachEntityChunkExecutionFunction);
		};
	}
}

void UMassTestProcessorBase::TestExecute(TSharedPtr<FMassEntityManager>& EntityManager)
{
	check(EntityManager);
	UMassTestProcessorBase* This = this;
	TArrayView<UMassProcessor* const> ProcessorView = MakeArrayView(reinterpret_cast<UMassProcessor**>(&This), 1);
	FMassProcessingContext ProcessingContext(EntityManager, /*DeltaSeconds=*/0.f);
	UE::Mass::Executor::RunProcessorsView(ProcessorView, ProcessingContext); 
}

void UMassTestProcessorBase::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	ExecutionFunction(EntityManager, Context);
}

void UMassTestProcessor_Floats::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
}

void UMassTestProcessor_Ints::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
}

void UMassTestProcessor_FloatsInts::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite);
}

 int UMassTestStaticCounterProcessor::StaticCounter = 0;
 UMassTestStaticCounterProcessor::UMassTestStaticCounterProcessor()
 {
#if WITH_EDITORONLY_DATA
	 bCanShowUpInSettings = false;
#endif // WITH_EDITORONLY_DATA
	 bAutoRegisterWithProcessingPhases = false;
	 ExecutionFlags = int32(EProcessorExecutionFlags::All);
 }

 UMassTestProcessorAutoExecuteQuery::UMassTestProcessorAutoExecuteQuery()
 {
	 bAutoRegisterWithProcessingPhases = false;
 }

//----------------------------------------------------------------------//
// UMassTestWorldSubsystem
//----------------------------------------------------------------------//
void UMassTestWorldSubsystem::Write(int32 InNumber)
{
	UE_MT_SCOPED_WRITE_ACCESS(AccessDetector);
	Number = InNumber;
}

int32 UMassTestWorldSubsystem::Read() const
{
	UE_MT_SCOPED_READ_ACCESS(AccessDetector);
	return Number;
}


namespace UE::Mass::Testing
{
//----------------------------------------------------------------------//
// FMassTestPhaseTickTask
//----------------------------------------------------------------------//
FMassTestPhaseTickTask::FMassTestPhaseTickTask(const TSharedRef<FMassProcessingPhaseManager>& InPhaseManager, const EMassProcessingPhase InPhase, const float InDeltaTime)
	: PhaseManager(InPhaseManager)
	, Phase(InPhase)
	, DeltaTime(InDeltaTime)
{
}

TStatId FMassTestPhaseTickTask::GetStatId()
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FMassTestPhaseTickTask, STATGROUP_TaskGraphTasks);
}

ENamedThreads::Type FMassTestPhaseTickTask::GetDesiredThread()
{ 
	return ENamedThreads::GameThread; 
}

ESubsequentsMode::Type FMassTestPhaseTickTask::GetSubsequentsMode()
{ 
	return ESubsequentsMode::TrackSubsequents; 
}

void FMassTestPhaseTickTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMassTestPhaseTickTask);
	PhaseManager->TriggerPhase(Phase, DeltaTime, MyCompletionGraphEvent);
}


//----------------------------------------------------------------------//
// FMassTestPhaseTickTask
//----------------------------------------------------------------------//
void FMassTestProcessingPhaseManager::Start(const TSharedPtr<FMassEntityManager>& InEntityManager)
{
	EntityManager = InEntityManager;

	OnNewArchetypeHandle = EntityManager->GetOnNewArchetypeEvent().AddRaw(this, &FMassTestProcessingPhaseManager::OnNewArchetype);

	// at this point FMassProcessingPhaseManager would call EnableTickFunctions if a world was available
	// here we're skipping it on purpose

	bIsAllowedToTick = true;
}

void FMassTestProcessingPhaseManager::OnNewArchetype(const FMassArchetypeHandle& NewArchetype)
{
	FMassProcessingPhaseManager::OnNewArchetype(NewArchetype);
}

} // namespace UE::Mass::Testing

UE_ENABLE_OPTIMIZATION_SHIP
