// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityTestTypes.h"
#include "TypeBitSetBuilder.h"
#include "MassBitSetRegistry.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::BitSetBuilder
{
	using namespace UE::Mass;

	template<typename Traits, typename TBuilderType>
	struct FBitSetBuilderTestBase : FEntityTestBase
	{
		using FTestElement_A = typename Traits::FTestElement_A;
		using FTestElement_B = typename Traits::FTestElement_B;
		using FTestElement_C = typename Traits::FTestElement_C;

		typename Traits::FBitRegistry BitRegistry;
		int32 TypeAIndex = INDEX_NONE;
		int32 TypeBIndex = INDEX_NONE;
		int32 TypeCIndex = INDEX_NONE;

		virtual bool SetUp() override
		{
			// register types ahead of type to ensure a known order
			TypeAIndex = BitRegistry.RegisterType(FTestElement_A::StaticStruct());
			TypeBIndex = BitRegistry.RegisterType(FTestElement_B::StaticStruct());
			TypeCIndex = BitRegistry.RegisterType(FTestElement_C::StaticStruct());

			return FEntityTestBase::SetUp();
		}

		virtual bool InstantTest() override
		{
			return MakeBuilderAndTest<TBuilderType>();
		}

		template<typename T>
		bool MakeBuilderAndTest();

		template<>
		bool MakeBuilderAndTest<typename Traits::FBitSetBuilder>()
		{
			typename Traits::FBitSet BitSet;
			typename Traits::FBitSetBuilder BitSetBuilder = BitRegistry.MakeBuilder(BitSet);
			return TestScenario(BitSetBuilder);
		}

		template<>
		bool MakeBuilderAndTest<typename Traits::FBitSetFactory>()
		{
			typename Traits::FBitSetFactory BitSetFactory = BitRegistry.MakeBuilder();
			return TestScenario(BitSetFactory);
		}

		bool TestScenario(typename Traits::FBitSetBuilder& BitSetBuilder)
		{
			BitSetBuilder.template Add<FTestElement_B>();

			typename Traits::FBitSetReader BitSetReader = BitSetBuilder;
			AITEST_EQUAL("The reading result is the same regardless of the method, existing element", BitSetReader.template Contains<FTestElement_B>(), BitSetReader.Contains(FTestElement_B::StaticStruct()));
			AITEST_EQUAL("The reading result is the same regardless of the method, non-existing element", BitSetReader.template Contains<FTestElement_A>(), BitSetReader.Contains(FTestElement_A::StaticStruct()));
			AITEST_TRUE("The resulting bitset has the right bit set", BitSetReader.template Contains<FTestElement_B>());
			AITEST_FALSE("(NOT) The resulting bitset has the wrong bit set", BitSetReader.template Contains<FTestElement_A>());

			const typename Traits::FBitSet BitSetCopy = BitSetBuilder;

			BitSetBuilder.template Add<FTestElement_B>();
			AITEST_EQUAL("Adding the same element doesn't change a thing", BitSetCopy, typename Traits::FBitSet(BitSetBuilder));

			BitSetBuilder.template Add<FTestElement_C>();
			AITEST_NOT_EQUAL("(NOT) Adding a different element doesn't change a thing", BitSetCopy, typename Traits::FBitSet(BitSetBuilder));

			BitSetBuilder.template Remove<FTestElement_C>();
			AITEST_EQUAL("Removing the added element makes bitsets equal again", BitSetCopy, typename Traits::FBitSet(BitSetBuilder));

			return true;
		}
	};

	namespace Fragments
	{
		struct FTraits
		{
			using FBitRegistry = FFragmentBitRegistry;
			using FBitSet = FMassFragmentBitSet_WIP;
			using FBitSetBuilder = FFragmentBitSetBuilder;
			using FBitSetReader = FFragmentBitSetReader;
			using FBitSetFactory = FFragmentBitSetFactory;
			using FTestElement_A = FTestFragment_Float;
			using FTestElement_B = FTestFragment_Int;
			using FTestElement_C = FTestFragment_Bool;
		};
		using FFragmentWrapper = FBitSetBuilderTestBase<FTraits, FTraits::FBitSetBuilder>;
		IMPLEMENT_AI_INSTANT_TEST(FFragmentWrapper, "System.Mass.BitSetBuilder.Wrapper.Fragments");
		using FFragmentStandalone = FBitSetBuilderTestBase<FTraits, FTraits::FBitSetFactory>;
		IMPLEMENT_AI_INSTANT_TEST(FFragmentStandalone, "System.Mass.BitSetBuilder.Standalone.Fragments");
	}

	namespace Tags
	{
		struct FTraits
		{
			using FBitRegistry = FTagBitRegistry;
			using FBitSet = FMassTagBitSet_WIP;
			using FBitSetBuilder = FTagBitSetBuilder;
			using FBitSetReader = FTagBitSetReader;
			using FBitSetFactory = FTagBitSetFactory;
			using FTestElement_A = FTestTag_A;
			using FTestElement_B = FTestTag_B;
			using FTestElement_C = FTestTag_C;
		};
		using FTagWrapper = FBitSetBuilderTestBase<FTraits, FTraits::FBitSetBuilder>;
		IMPLEMENT_AI_INSTANT_TEST(FTagWrapper, "System.Mass.BitSetBuilder.Wrapper.Tags");
		using FTagStandalone = FBitSetBuilderTestBase<FTraits, FTraits::FBitSetFactory>;
		IMPLEMENT_AI_INSTANT_TEST(FTagStandalone, "System.Mass.BitSetBuilder.Standalone.Tags");
	}
} // UE::Mass::Test::TypeRegistry

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
