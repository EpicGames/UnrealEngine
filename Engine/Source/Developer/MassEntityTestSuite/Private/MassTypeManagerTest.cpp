// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "Engine/Engine.h"
#include "MassEntityTestTypes.h"
#include "MassTypeManager.h"
#include "MassExecutionContext.h"
#include "MassSubsystemAccess.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test
{
	/** Helper to create, initialize, and post-initialize a fresh FMassEntityManager. */
	TSharedRef<FMassEntityManager> MakeInitializedEntityManager(const bool bCallPostInitialize = true)
	{
		TSharedRef<FMassEntityManager> Manager = MakeShareable(new FMassEntityManager(FAITestHelpers::GetWorld()));
		FMassEntityManagerStorageInitParams InitParams;
		InitParams.Emplace<FMassEntityManager_InitParams_Concurrent>();
		Manager->Initialize(InitParams);
		if (bCallPostInitialize)
		{
			Manager->PostInitialize();
		}
		return Manager;
	}

	struct FTypeManager_Subsystem : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const FSubsystemTypeTraits TestSubsystemTraits = FSubsystemTypeTraits::Make<UMassTestWorldSubsystem>();

			AITEST_EQUAL("Subsystem bGameThreadOnly value", TestSubsystemTraits.bGameThreadOnly, TMassExternalSubsystemTraits<UMassTestWorldSubsystem>::GameThreadOnly)
			AITEST_EQUAL("Subsystem ThreadSafeWrite value", TestSubsystemTraits.bThreadSafeWrite, TMassExternalSubsystemTraits<UMassTestWorldSubsystem>::ThreadSafeWrite)

			const FSharedFragmentTypeTraits TestSharedFragmentTraits = FSharedFragmentTypeTraits::Make<FTestSharedFragment_Int>();
			AITEST_EQUAL("Shared fragment bGameThreadOnly value", TestSharedFragmentTraits.bGameThreadOnly, TMassSharedFragmentTraits<FTestSharedFragment_Int>::GameThreadOnly)

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_Subsystem, "System.Mass.TypeManager.StaticSubsystem");

	//-------------------------------------------------------------------------
	// Static type registration tests
	//-------------------------------------------------------------------------

	/** Verify that a statically registered subsystem type appears in a new TypeManager after RegisterBuiltInTypes. */
	struct FTypeManager_StaticRegistration_Subsystem : FExecutionTestBase
	{
		virtual bool InstantTest() override
		{
			CA_ASSUME(EntityManager);

			// Register a subsystem type statically via the RAII handle
			FTypeManager::FStaticTypeRegistrationHandle Handle = FTypeManager::RegisterStaticType<UMassTestWorldSubsystem>();
			AITEST_TRUE("Handle is valid after registration", Handle.IsValid());

			// Create a fresh entity manager and initialize it — this triggers RegisterBuiltInTypes
			TSharedRef<FMassEntityManager> TestEntityManager = MakeInitializedEntityManager();

			// The statically registered type should be in the new TypeManager
			const FTypeHandle TypeHandle = FTypeManager::MakeTypeHandle(UMassTestWorldSubsystem::StaticClass());
			const FTypeInfo* TypeInfo = TestEntityManager->GetTypeManager().GetTypeInfo(TypeHandle);
			AITEST_NOT_NULL("Statically registered subsystem type found in TypeManager", TypeInfo);
			AITEST_NOT_NULL("TypeInfo has subsystem traits", TypeInfo ? TypeInfo->GetAsSystemTraits() : nullptr);

			TestEntityManager->Deinitialize();
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_Subsystem, "System.Mass.TypeManager.StaticRegistration.Subsystem");

	/** Verify that a statically registered shared fragment type appears in a new TypeManager after RegisterBuiltInTypes. */
	struct FTypeManager_StaticRegistration_SharedFragment : FExecutionTestBase
	{
		virtual bool InstantTest() override
		{
			CA_ASSUME(EntityManager);

			FTypeManager::FStaticTypeRegistrationHandle Handle = FTypeManager::RegisterStaticType<FTestSharedFragment_Int>();
			AITEST_TRUE("Handle is valid after registration", Handle.IsValid());

			TSharedRef<FMassEntityManager> TestEntityManager = MakeInitializedEntityManager();

			const FTypeHandle TypeHandle = FTypeManager::MakeTypeHandle(FTestSharedFragment_Int::StaticStruct());
			const FTypeInfo* TypeInfo = TestEntityManager->GetTypeManager().GetTypeInfo(TypeHandle);
			AITEST_NOT_NULL("Statically registered shared fragment type found in TypeManager", TypeInfo);
			AITEST_NOT_NULL("TypeInfo has shared fragment traits", TypeInfo ? TypeInfo->GetAsSharedFragmentTraits() : nullptr);

			TestEntityManager->Deinitialize();
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_SharedFragment, "System.Mass.TypeManager.StaticRegistration.SharedFragment");

	/** Verify that destroying the RAII handle removes the type from the static registry. */
	struct FTypeManager_StaticRegistration_HandleDeregisters : FExecutionTestBase
	{
		virtual bool InstantTest() override
		{
			CA_ASSUME(EntityManager);

			{
				FTypeManager::FStaticTypeRegistrationHandle Handle = FTypeManager::RegisterStaticType<FTestSharedFragment_Int>();
				// Handle goes out of scope and should deregister
			}

			// A new entity manager should NOT see the type
			TSharedRef<FMassEntityManager> TestEntityManager = MakeInitializedEntityManager();
			
			const FTypeHandle TypeHandle = FTypeManager::MakeTypeHandle(FTestSharedFragment_Int::StaticStruct());
			const FTypeInfo* TypeInfo = TestEntityManager->GetTypeManager().GetTypeInfo(TypeHandle);
			AITEST_NULL("Deregistered type should not be in new TypeManager", TypeInfo);

			TestEntityManager->Deinitialize();
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_HandleDeregisters, "System.Mass.TypeManager.StaticRegistration.HandleDeregisters");

	/** Verify that handle move semantics work: moved-from handle does not deregister. */
	struct FTypeManager_StaticRegistration_HandleMoveSemantics : FExecutionTestBase
	{
		virtual bool InstantTest() override
		{
			CA_ASSUME(EntityManager);

			FTypeManager::FStaticTypeRegistrationHandle TargetHandle;
			{
				FTypeManager::FStaticTypeRegistrationHandle SourceHandle = FTypeManager::RegisterStaticType<FTestSharedFragment_Int>();
				AITEST_TRUE("Source handle is valid", SourceHandle.IsValid());

				TargetHandle = MoveTemp(SourceHandle);
				AITEST_FALSE("Moved-from handle is invalid", SourceHandle.IsValid());
				AITEST_TRUE("Move target handle is valid", TargetHandle.IsValid());
				// SourceHandle goes out of scope — should NOT deregister
			}

			// Type should still be in the static registry because TargetHandle holds ownership
			TSharedRef<FMassEntityManager> TestEntityManager = MakeInitializedEntityManager();

			const FTypeHandle TypeHandle = FTypeManager::MakeTypeHandle(FTestSharedFragment_Int::StaticStruct());
			const FTypeInfo* TypeInfo = TestEntityManager->GetTypeManager().GetTypeInfo(TypeHandle);
			AITEST_NOT_NULL("Type still registered after source handle destroyed", TypeInfo);

			TestEntityManager->Deinitialize();

			// Now destroy target handle — cleanup
			TargetHandle = FTypeManager::FStaticTypeRegistrationHandle();
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_HandleMoveSemantics, "System.Mass.TypeManager.StaticRegistration.HandleMoveSemantics");

	//-------------------------------------------------------------------------
	// Static registration automation tests
	//-------------------------------------------------------------------------

	/** Static registration survives across multiple EntityManager lifecycles. */
	struct FTypeManager_StaticRegistration_MultiLifecycle : FExecutionTestBase
	{
		virtual bool InstantTest() override
		{
			FTypeManager::FStaticTypeRegistrationHandle Handle = FTypeManager::RegisterStaticType<UMassTestEngineSubsystem>();
			const FTypeHandle TypeHandle = FTypeManager::MakeTypeHandle(UMassTestEngineSubsystem::StaticClass());

			// First entity manager lifecycle
			{
				TSharedRef<FMassEntityManager> Manager1 = MakeInitializedEntityManager();
				const FTypeInfo* TypeInfo = Manager1->GetTypeManager().GetTypeInfo(TypeHandle);
				AITEST_NOT_NULL("Type present in first EntityManager", TypeInfo);
				AITEST_NOT_NULL("First manager has subsystem traits", TypeInfo ? TypeInfo->GetAsSystemTraits() : nullptr);
				Manager1->Deinitialize();
			}

			// Second entity manager lifecycle — static registration should still be there
			{
				TSharedRef<FMassEntityManager> Manager2 = MakeInitializedEntityManager();
				const FTypeInfo* TypeInfo = Manager2->GetTypeManager().GetTypeInfo(TypeHandle);
				AITEST_NOT_NULL("Type present in second EntityManager", TypeInfo);
				Manager2->Deinitialize();
			}

			// Third — for good measure
			{
				TSharedRef<FMassEntityManager> Manager3 = MakeInitializedEntityManager();
				const FTypeInfo* TypeInfo = Manager3->GetTypeManager().GetTypeInfo(TypeHandle);
				AITEST_NOT_NULL("Type present in third EntityManager", TypeInfo);
				Manager3->Deinitialize();
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_MultiLifecycle, "System.Mass.TypeManager.StaticRegistration.MultiLifecycle");

	/** Delegate handler overrides statically registered type (delegate runs after static drain). */
	struct FTypeManager_StaticRegistration_DelegateOverridesStatic : FExecutionTestBase
	{
		FTypeManager::FStaticTypeRegistrationHandle StaticHandle;
		FDelegateHandle BuiltInTypesDelegateHandle;

		virtual bool SetUp() override
		{
			if (!FExecutionTestBase::SetUp())
			{
				return false;
			}

			StaticHandle = FTypeManager::RegisterStaticType<UMassTestWorldSubsystem>();
			BuiltInTypesDelegateHandle = FTypeManager::OnRegisterBuiltInTypes.AddLambda([](FTypeManager& TypeMgr)
			{
				FSubsystemTypeTraits OverriddenTraits;
				OverriddenTraits.bGameThreadOnly = false;
				OverriddenTraits.bThreadSafeWrite = true;
				TypeMgr.RegisterType(UMassTestWorldSubsystem::StaticClass(), MoveTemp(OverriddenTraits));
			});
			return true;
		}

		virtual void TearDown() override
		{
			FTypeManager::OnRegisterBuiltInTypes.Remove(BuiltInTypesDelegateHandle);
			StaticHandle = FTypeManager::FStaticTypeRegistrationHandle();
			FExecutionTestBase::TearDown();
		}

		virtual bool InstantTest() override
		{
			TSharedRef<FMassEntityManager> TestManager = MakeInitializedEntityManager();

			const FTypeHandle TypeHandle = FTypeManager::MakeTypeHandle(UMassTestWorldSubsystem::StaticClass());
			const FTypeInfo* TypeInfo = TestManager->GetTypeManager().GetTypeInfo(TypeHandle);
			AITEST_NOT_NULL("Type is registered", TypeInfo);

			const FSubsystemTypeTraits* Traits = TypeInfo ? TypeInfo->GetAsSystemTraits() : nullptr;
			AITEST_NOT_NULL("Has subsystem traits", Traits);
			if (Traits)
			{
				// Delegate runs AFTER static drain, so the delegate's traits should win
				AITEST_FALSE("Delegate override: bGameThreadOnly should be false", Traits->bGameThreadOnly);
				AITEST_TRUE("Delegate override: bThreadSafeWrite should be true", Traits->bThreadSafeWrite);
			}

			TestManager->Deinitialize();
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_DelegateOverridesStatic, "System.Mass.TypeManager.StaticRegistration.DelegateOverridesStatic");

	/** Instance-level RegisterType (from subsystem Initialize) and static registration coexist.
	 * Per-instance registration happens before PostInitialize, static drain happens during PostInitialize.
	 * Static drain overwrites the per-instance data (it runs in RegisterBuiltInTypes which is after Initialize). */
	struct FTypeManager_StaticRegistration_OverridePrecedence : FExecutionTestBase
	{
		virtual bool InstantTest() override
		{
			// Register statically with specific traits
			FSubsystemTypeTraits StaticTraits;
			StaticTraits.bGameThreadOnly = false;
			StaticTraits.bThreadSafeWrite = true;
			FTypeManager::FStaticTypeRegistrationHandle Handle = FTypeManager::RegisterStaticType(
				UMassTestEngineSubsystem::StaticClass(), MoveTemp(StaticTraits));

			// Create a manager, register the same type with different traits BEFORE PostInitialize
			TSharedRef<FMassEntityManager> TestManager = MakeInitializedEntityManager(/*bCallPostInitialize=*/false);

			// Simulate what a subsystem's Initialize() would do — register before PostInitialize
			FSubsystemTypeTraits InstanceTraits;
			InstanceTraits.bGameThreadOnly = true;
			InstanceTraits.bThreadSafeWrite = false;
			TestManager->GetTypeManager().RegisterType(UMassTestEngineSubsystem::StaticClass(), MoveTemp(InstanceTraits));

			// Now PostInitialize — this drains static types (which re-registers with static traits)
			TestManager->PostInitialize();

			const FTypeHandle TypeHandle = FTypeManager::MakeTypeHandle(UMassTestEngineSubsystem::StaticClass());
			const FTypeInfo* TypeInfo = TestManager->GetTypeManager().GetTypeInfo(TypeHandle);
			AITEST_NOT_NULL("Type is registered", TypeInfo);

			const FSubsystemTypeTraits* Traits = TypeInfo ? TypeInfo->GetAsSystemTraits() : nullptr;
			AITEST_NOT_NULL("Has subsystem traits", Traits);
			if (Traits)
			{
				// Static drain runs during PostInitialize, AFTER instance-level registration,
				// so the static traits overwrite the instance traits
				AITEST_FALSE("Static override wins: bGameThreadOnly should be false", Traits->bGameThreadOnly);
				AITEST_TRUE("Static override wins: bThreadSafeWrite should be true", Traits->bThreadSafeWrite);
			}

			TestManager->Deinitialize();
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_OverridePrecedence, "System.Mass.TypeManager.StaticRegistration.OverridePrecedence");

	/** Processor can declare a subsystem requirement on a statically registered type and resolve it. */
	struct FTypeManager_StaticRegistration_ProcessorSubsystemRequirement : FExecutionTestBase
	{
		FTypeManager_StaticRegistration_ProcessorSubsystemRequirement()
		{
			bMakeWorldEntityManagersOwner = true;
		}

		virtual bool InstantTest() override
		{
			CA_ASSUME(EntityManager);

			// The test's own EntityManager is already initialized.
			// Register the engine subsystem type statically so it's known to the type system.
			FTypeManager::FStaticTypeRegistrationHandle Handle = FTypeManager::RegisterStaticType<UMassTestEngineSubsystem>();

			// Re-register into the existing EntityManager's TypeManager so it sees it too
			// (the existing manager was already PostInitialized before our static registration)
			EntityManager->GetTypeManager().RegisterType<UMassTestEngineSubsystem>();

			// Set up a query with a subsystem requirement
			FMassEntityQuery EntityQuery(EntityManager.ToSharedRef());
			EntityQuery.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly);
			EntityQuery.AddSubsystemRequirement<UMassTestEngineSubsystem>(EMassFragmentAccess::ReadOnly);

			// Create an entity so the query has something to iterate
			const UScriptStruct* FragmentList[] = { FTestFragment_Float::StaticStruct() };
			FMassArchetypeHandle Archetype = EntityManager->CreateArchetype(FragmentList);
			EntityManager->CreateEntity(Archetype);

			// Execute the query — verify the subsystem is accessible
			UMassTestEngineSubsystem* ActualSubsystem = GEngine ? GEngine->GetEngineSubsystem<UMassTestEngineSubsystem>() : nullptr;
			const UMassTestEngineSubsystem* AccessedSubsystem = nullptr;
			bool bQueryExecuted = false;

			FMassExecutionContext ExecutionContext(*EntityManager);
			EntityQuery.ForEachEntityChunk(ExecutionContext, [&](FMassExecutionContext& Context)
			{
				bQueryExecuted = true;
				AccessedSubsystem = Context.GetSubsystem<UMassTestEngineSubsystem>();
			});

			AITEST_TRUE("Query executed at least once", bQueryExecuted);
			if (ActualSubsystem)
			{
				AITEST_EQUAL("Accessed subsystem matches actual engine subsystem", ActualSubsystem, AccessedSubsystem);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_ProcessorSubsystemRequirement, "System.Mass.TypeManager.StaticRegistration.ProcessorSubsystemRequirement");

	/** Assigning a new registration to an existing handle deregisters the old type. */
	struct FTypeManager_StaticRegistration_HandleReassignment : FExecutionTestBase
	{
		virtual bool InstantTest() override
		{
			// Register shared fragment first
			FTypeManager::FStaticTypeRegistrationHandle Handle = FTypeManager::RegisterStaticType<FTestSharedFragment_Int>();
			const FTypeHandle SharedFragHandle = FTypeManager::MakeTypeHandle(FTestSharedFragment_Int::StaticStruct());

			// Now reassign the handle to hold a subsystem type instead
			Handle = FTypeManager::RegisterStaticType<UMassTestEngineSubsystem>();
			const FTypeHandle SubsystemHandle = FTypeManager::MakeTypeHandle(UMassTestEngineSubsystem::StaticClass());

			// Create a new manager and verify: old type gone, new type present
			TSharedRef<FMassEntityManager> TestManager = MakeInitializedEntityManager();

			const FTypeInfo* SharedFragInfo = TestManager->GetTypeManager().GetTypeInfo(SharedFragHandle);
			AITEST_NULL("Old type (shared fragment) should be deregistered", SharedFragInfo);

			const FTypeInfo* SubsystemInfo = TestManager->GetTypeManager().GetTypeInfo(SubsystemHandle);
			AITEST_NOT_NULL("New type (subsystem) should be registered", SubsystemInfo);
			AITEST_NOT_NULL("New type has subsystem traits", SubsystemInfo ? SubsystemInfo->GetAsSystemTraits() : nullptr);

			TestManager->Deinitialize();
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTypeManager_StaticRegistration_HandleReassignment, "System.Mass.TypeManager.StaticRegistration.HandleReassignment");
} // namespace UE::Mass::Test

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE