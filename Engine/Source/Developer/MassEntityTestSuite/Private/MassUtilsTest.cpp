// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// tests 
//----------------------------------------------------------------------//

namespace FMassUtilsTest
{

struct FUtilsTest_MultiSort : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<float> ValuesF = { 1,2,3,4,5,6,7,8,9,10 };
		TArray<bool> ValuesB = { 1,0,1,0,1,0,1,0,1,0 };
		TArray<int8> ValuesU = { -1,2,-3,4,-5,6,-7,8,-9,10 };

		UE::Mass::Utils::AbstractSort(ValuesU.Num(), [&ValuesU](const int32 LHS, const int32 RHS)
			{
				return ValuesU[LHS] < ValuesU[RHS];
			}
			, [&ValuesF, &ValuesB, &ValuesU](const int32 A, const int32 B)
			{
				Swap(ValuesF[A], ValuesF[B]);
				Swap(ValuesB[A], ValuesB[B]);
				Swap(ValuesU[A], ValuesU[B]);
			});

		for (int i = 0; i < ValuesF.Num(); ++i)
		{
			AITEST_EQUAL(TEXT("Element in ValuesU should represent the absolute value of the element in ValuesF"), float(FMath::Abs(ValuesU[i])), ValuesF[i]);
			AITEST_EQUAL(TEXT("Element in ValuesB should represent the sign of the element in ValuesU"), ValuesU[i] < 0, ValuesB[i]);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FUtilsTest_MultiSort, "System.Mass.Utils.MultiSort");

struct FUtilsTest_MultiSort_GenericArray : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FTestFragment_Float> ValuesF = { 1,2,3,4,5,6,7,8,9,10 };
		TArray<FTestFragment_Bool> ValuesB = { 1,0,1,0,1,0,1,0,1,0 };
		TArray<int8> ValuesU = { -1,2,-3,4,-5,6,-7,8,-9,10 };
		FStructArrayView ViewF(ValuesF);
		FStructArrayView ViewB(ValuesB);

		UE::Mass::Utils::AbstractSort(ValuesU.Num(), [&ValuesU](const int32 LHS, const int32 RHS)
			{
				return ValuesU[LHS] < ValuesU[RHS];
			}
			, [&ViewF, &ViewB, &ValuesU](const int32 A, const int32 B)
			{
				Swap(ValuesU[A], ValuesU[B]);
				ViewF.Swap(A, B);
				ViewB.Swap(A, B);
			});

		for (int i = 0; i < ValuesF.Num(); ++i)
		{
			AITEST_EQUAL(TEXT("Element in ValuesU should represent the absolute value of the element in ViewF"), float(FMath::Abs(ValuesU[i])), ViewF.GetAt<const FTestFragment_Float>(i).Value);
			AITEST_EQUAL(TEXT("Element in ViewB should represent the sign of the element in ValuesU"), ValuesU[i] < 0, ViewB.GetAt<const FTestFragment_Bool>(i).bValue);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FUtilsTest_MultiSort_GenericArray, "System.Mass.Utils.MultiSort_GenericArray");

struct FUtilsTest_MultiSort_Payload : FExecutionTestBase
{
	virtual bool InstantTest() override
	{
		TArray<FTestFragment_Float> ValuesF = { 1,2,3,4,5,6,7,8,9,10 };
		TArray<FTestFragment_Bool> ValuesB = { 1,0,1,0,1,0,1,0,1,0 };
		TArray<int8> ValuesU = { -1,2,-3,4,-5,6,-7,8,-9,10 };
		FStructArrayView ViewF(ValuesF);
		FStructArrayView ViewB(ValuesB);
		TArray<FStructArrayView> PayloadArray = { ViewF , ViewB };
		FMassGenericPayloadView Payload(PayloadArray);

		UE::Mass::Utils::AbstractSort(ValuesU.Num(), [&ValuesU](const int32 LHS, const int32 RHS)
			{
				return ValuesU[LHS] < ValuesU[RHS];
			}
			, [Payload, &ValuesU](const int32 A, const int32 B) mutable
			{
				Swap(ValuesU[A], ValuesU[B]);
				Payload.Swap(A, B);
			});

		for (int i = 0; i < ValuesF.Num(); ++i)
		{
			AITEST_EQUAL(TEXT("Element in ValuesU should represent the absolute value of the element in ViewF"), float(FMath::Abs(ValuesU[i])), ViewF.GetAt<const FTestFragment_Float>(i).Value);
			AITEST_EQUAL(TEXT("Element in ViewB should represent the sign of the element in ValuesU"), ValuesU[i] < 0, ViewB.GetAt<const FTestFragment_Bool>(i).bValue);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FUtilsTest_MultiSort_Payload, "System.Mass.Utils.MultiSort_Payload");

} // FMassUtilsTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
