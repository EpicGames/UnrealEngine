// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "Algo/RandomShuffle.h"
#include "MassCommandBuffer.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

//----------------------------------------------------------------------//
// Sparse Element Command Tests
//----------------------------------------------------------------------//

namespace UE::Mass::Test::Commands
{
#if WITH_MASSENTITY_DEBUG

struct FCommands_AddSparseFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add sparse fragments via command
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
		}

		// Verify sparse fragments not yet added
		for (int i = 0; i < Count; ++i)
		{
			AITEST_FALSE(TEXT("Sparse fragment should not exist before flush"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
		}

		EntityManager->FlushCommands();

		// Verify all entities have sparse fragments after flush
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Sparse fragment should exist after flush"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

			// Verify entities stayed in same archetype
			AITEST_EQUAL(TEXT("Entity should remain in original archetype"),
				EntityManager->GetArchetypeForEntity(Entities[i]), IntsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddSparseFragment, "System.Mass.Stress.Commands.SparseElements.AddSparseFragment");


struct FCommands_AddSparseTag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

		// Add sparse tags via command
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestTag_SparseA::StaticStruct()).Add(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify all entities have sparse tags
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Sparse tag should exist after flush"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));

			// Verify entities stayed in same archetype
			AITEST_EQUAL(TEXT("Entity should remain in original archetype"),
				EntityManager->GetArchetypeForEntity(Entities[i]), FloatsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_AddSparseTag, "System.Mass.Stress.Commands.SparseElements.AddSparseTag");


struct FCommands_RemoveSparseFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

		// Add sparse fragments directly
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
		}

		// Verify all have sparse fragments
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Sparse fragment should exist"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
		}

		// Remove via command
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
				FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify all sparse fragments removed
		for (int i = 0; i < Count; ++i)
		{
			AITEST_FALSE(TEXT("Sparse fragment should be removed after flush"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

			// Verify entities stayed in same archetype
			AITEST_EQUAL(TEXT("Entity should remain in original archetype"),
				EntityManager->GetArchetypeForEntity(Entities[i]), FloatsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveSparseFragment, "System.Mass.Stress.Commands.SparseElements.RemoveSparseFragment");


struct FCommands_RemoveSparseTag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add sparse tags directly
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
		}

		// Remove via command
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
				FTestTag_SparseA::StaticStruct()).Add(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify all sparse tags removed
		for (int i = 0; i < Count; ++i)
		{
			AITEST_FALSE(TEXT("Sparse tag should be removed after flush"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_RemoveSparseTag, "System.Mass.Stress.Commands.SparseElements.RemoveSparseTag");


struct FCommands_BatchAddSparseFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f); // Span multiple chunks

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);

		// Batch add sparse fragments via single command with multiple entities
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseInt::StaticStruct()).Add(Entities);

		EntityManager->FlushCommands();

		// Verify all entities have sparse fragments
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("All entities should have sparse fragment after batch add"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

			// Verify entities stayed in same archetype
			AITEST_EQUAL(TEXT("Entity should remain in original archetype"),
				EntityManager->GetArchetypeForEntity(Entities[i]), FloatsIntsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchAddSparseFragments, "System.Mass.Stress.Commands.SparseElements.BatchAddSparseFragments");


struct FCommands_BatchRemoveSparseFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f);

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add sparse fragments to all entities
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
		}

		// Batch remove via single command
		EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
			FTestFragment_SparseFloat::StaticStruct()).Add(Entities);

		EntityManager->FlushCommands();

		// Verify all sparse fragments removed
		for (int i = 0; i < Count; ++i)
		{
			AITEST_FALSE(TEXT("All sparse fragments should be removed after batch remove"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchRemoveSparseFragments, "System.Mass.Stress.Commands.SparseElements.BatchRemoveSparseFragments");


struct FCommands_MixedSparseAndRegularElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Mix regular fragment addition with sparse element addition
		for (int i = 0; i < Count; ++i)
		{
			// Add regular fragment (causes archetype change)
			EntityManager->Defer().AddFragment<FTestFragment_Float>(Entities[i]);
			// Add sparse fragment (no archetype change)
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify entities moved to new archetype and have sparse elements
		for (int i = 0; i < Count; ++i)
		{
			AITEST_EQUAL(TEXT("Entity should be in FloatsInts archetype"),
				EntityManager->GetArchetypeForEntity(Entities[i]), FloatsIntsArchetype);
			AITEST_TRUE(TEXT("Entity should have sparse fragment"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

			// Verify regular fragment accessible
			AITEST_NOT_NULL(TEXT("Entity should have float fragment"),
				EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entities[i]));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_MixedSparseAndRegularElements, "System.Mass.Stress.Commands.SparseElements.MixedSparseAndRegular");


struct FCommands_SparseFragmentDataPreservation : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

		// Add sparse fragments and set values
		for (int i = 0; i < Count; ++i)
		{
			FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
			SparseData.Value = i * 100;
		}

		// Trigger archetype change by adding regular fragment
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().AddFragment<FTestFragment_Int>(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify sparse fragment data preserved across archetype change
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Sparse fragment should still exist"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

			FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
				FTestFragment_SparseInt::StaticStruct(), Entities[i]);
			AITEST_EQUAL(TEXT("Sparse fragment data should be preserved"),
				SparseData.Get<FTestFragment_SparseInt>().Value, i * 100);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_SparseFragmentDataPreservation, "System.Mass.Stress.Commands.SparseElements.DataPreservation");


struct FCommands_SparseDuplicateAddHandling : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add same sparse fragment multiple times (should be idempotent)
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestFragment_SparseFloat::StaticStruct()).Add(Entities[i]);
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestFragment_SparseFloat::StaticStruct()).Add(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify sparse fragment exists (should handle duplicates gracefully)
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Sparse fragment should exist after duplicate add"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_SparseDuplicateAddHandling, "System.Mass.Stress.Commands.SparseElements.DuplicateAddHandling");


struct FCommands_SparseRemoveNonExistent : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 5;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

		// Try to remove sparse fragments that don't exist (should be safe)
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushUniqueCommand<FMassCommandRemoveElement>(
				FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify entities still valid and in original archetype
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Entity should still be valid"), EntityManager->IsEntityValid(Entities[i]));
			AITEST_EQUAL(TEXT("Entity should remain in original archetype"),
				EntityManager->GetArchetypeForEntity(Entities[i]), FloatsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_SparseRemoveNonExistent, "System.Mass.Stress.Commands.SparseElements.RemoveNonExistent");


struct FCommands_SparseMultipleTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add multiple sparse element types to same entities
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestFragment_SparseInt::StaticStruct()).Add(Entities[i]);
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestFragment_SparseFloat::StaticStruct()).Add(Entities[i]);
			EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
				FTestTag_SparseA::StaticStruct()).Add(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify all sparse element types exist
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("SparseInt should exist"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
			AITEST_TRUE(TEXT("SparseFloat should exist"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
			AITEST_TRUE(TEXT("SparseTag should exist"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));

			// All on same archetype
			AITEST_EQUAL(TEXT("Entity should remain in original archetype"),
				EntityManager->GetArchetypeForEntity(Entities[i]), IntsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_SparseMultipleTypes, "System.Mass.Stress.Commands.SparseElements.MultipleTypes");


struct FCommands_SparseAcrossArchetypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> IntEntities;
		TArray<FMassEntityHandle> FloatEntities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, IntEntities);
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, FloatEntities);

		TArray<FMassEntityHandle> AllEntities;
		AllEntities.Append(IntEntities);
		AllEntities.Append(FloatEntities);

		// Add same sparse element to entities from different archetypes
		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseInt::StaticStruct()).Add(AllEntities);

		EntityManager->FlushCommands();

		// Verify all entities have the sparse element regardless of archetype
		for (const FMassEntityHandle& Entity : IntEntities)
		{
			AITEST_TRUE(TEXT("IntArchetype entity should have sparse fragment"),
				EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			AITEST_EQUAL(TEXT("Entity should remain in Ints archetype"),
				EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);
		}

		for (const FMassEntityHandle& Entity : FloatEntities)
		{
			AITEST_TRUE(TEXT("FloatArchetype entity should have sparse fragment"),
				EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			AITEST_EQUAL(TEXT("Entity should remain in Floats archetype"),
				EntityManager->GetArchetypeForEntity(Entity), FloatsArchetype);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_SparseAcrossArchetypes, "System.Mass.Stress.Commands.SparseElements.AcrossArchetypes");


struct FCommands_SparseWithEntityDestruction : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add sparse fragments
		for (int i = 0; i < Count; ++i)
		{
			EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
			FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
				FTestFragment_SparseInt::StaticStruct(), Entities[i]);
			SparseData.Get<FTestFragment_SparseInt>().Value = i;
		}

		// Destroy half the entities via command
		TArray<FMassEntityHandle> EntitiesToDestroy;
		for (int i = 0; i < Count / 2; ++i)
		{
			EntitiesToDestroy.Add(Entities[i]);
			EntityManager->Defer().DestroyEntity(Entities[i]);
		}

		EntityManager->FlushCommands();

		// Verify destroyed entities are invalid
		for (int i = 0; i < Count / 2; ++i)
		{
			AITEST_FALSE(TEXT("Destroyed entity should be invalid"),
				EntityManager->IsEntityValid(Entities[i]));
		}

		// Verify remaining entities still have sparse fragments with correct data
		for (int i = Count / 2; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Remaining entity should still be valid"),
				EntityManager->IsEntityValid(Entities[i]));
			AITEST_TRUE(TEXT("Remaining entity should still have sparse fragment"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));

			FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
				FTestFragment_SparseInt::StaticStruct(), Entities[i]);
			AITEST_EQUAL(TEXT("Sparse data should still be valid"),
				SparseData.Get<FTestFragment_SparseInt>().Value, i);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_SparseWithEntityDestruction, "System.Mass.Stress.Commands.SparseElements.WithEntityDestruction");


struct FCommands_SparsePartialBatch : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 20;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

		// Add sparse fragment to first half only
		TArray<FMassEntityHandle> FirstHalf;
		for (int i = 0; i < Count / 2; ++i)
		{
			FirstHalf.Add(Entities[i]);
		}

		EntityManager->Defer().PushUniqueCommand<FMassCommandAddElement>(
			FTestFragment_SparseFloat::StaticStruct()).Add(FirstHalf);

		EntityManager->FlushCommands();

		// Verify first half has sparse fragments
		for (int i = 0; i < Count / 2; ++i)
		{
			AITEST_TRUE(TEXT("First half should have sparse fragment"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
		}

		// Verify second half does not have sparse fragments
		for (int i = Count / 2; i < Count; ++i)
		{
			AITEST_FALSE(TEXT("Second half should not have sparse fragment"),
				EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseFloat::StaticStruct()));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_SparsePartialBatch, "System.Mass.Stress.Commands.SparseElements.PartialBatch");


struct FCommands_BatchDestroyEntities_WithSparseElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 20;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, Entities);

		// Add sparse fragments and tags to all entities
		for (int i = 0; i < Count; ++i)
		{
			FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
			SparseData.Value = i * 10;

			EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
		}

		// Verify sparse elements exist in storage
		const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
		AITEST_EQUAL(TEXT("Storage should have all SparseInt elements"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), static_cast<uint32>(Count));

		// Verify each entity has sparse elements
		for (int i = 0; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Entity should have SparseInt"),
				EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entities[i]));
		}

		// Destroy all entities using BatchDestroyEntities
		EntityManager->BatchDestroyEntities(Entities);

		// Verify all entities are invalid
		for (int i = 0; i < Count; ++i)
		{
			AITEST_FALSE(TEXT("Entity should be invalid after destruction"),
				EntityManager->IsEntityValid(Entities[i]));
		}

		// Verify sparse elements removed from storage
		AITEST_EQUAL(TEXT("Storage should have no SparseInt elements after destruction"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), 0u);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchDestroyEntities_WithSparseElements, "System.Mass.Stress.Commands.BatchDestroy.WithSparseElements");


struct FCommands_BatchDestroyEntities_PartialDestruction : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 Count = 30;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, Count, Entities);

		// Add sparse fragments to all entities with unique values
		for (int i = 0; i < Count; ++i)
		{
			FTestFragment_SparseFloat& SparseFloat = EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
			SparseFloat.Value = static_cast<float>(i);
		}

		const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
		AITEST_EQUAL(TEXT("Storage should have all elements initially"),
			Storage.GetNumElementsOfType<FTestFragment_SparseFloat>(), static_cast<uint32>(Count));

		// Destroy first third of entities
		TArray<FMassEntityHandle> EntitiesToDestroy;
		for (int i = 0; i < Count / 3; ++i)
		{
			EntitiesToDestroy.Add(Entities[i]);
		}

		EntityManager->BatchDestroyEntities(EntitiesToDestroy);

		// Verify destroyed entities are invalid
		for (int i = 0; i < Count / 3; ++i)
		{
			AITEST_FALSE(TEXT("Destroyed entity should be invalid"),
				EntityManager->IsEntityValid(Entities[i]));
		}

		// Verify remaining entities still valid with correct data
		const int32 ExpectedRemaining = Count - Count / 3;
		AITEST_EQUAL(TEXT("Storage should have correct number of remaining elements"),
			Storage.GetNumElementsOfType<FTestFragment_SparseFloat>(), static_cast<uint32>(ExpectedRemaining));

		for (int i = Count / 3; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Remaining entity should still be valid"),
				EntityManager->IsEntityValid(Entities[i]));
			AITEST_TRUE(TEXT("Remaining entity should still have sparse element"),
				EntityManager->DoesEntityHaveElement<FTestFragment_SparseFloat>(Entities[i]));

			FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
				FTestFragment_SparseFloat::StaticStruct(), Entities[i]);
			AITEST_EQUAL(TEXT("Sparse data should be preserved for remaining entities"),
				SparseData.Get<FTestFragment_SparseFloat>().Value, static_cast<float>(i));
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchDestroyEntities_PartialDestruction, "System.Mass.Stress.Commands.BatchDestroy.PartialDestruction");


struct FCommands_BatchDestroyEntityChunks_WithSparseElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsIntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f); // Span multiple chunks

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);

		// Add multiple sparse element types to all entities
		for (int i = 0; i < Count; ++i)
		{
			FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
			SparseInt.Value = i;

			FTestFragment_SparseFloat& SparseFloat = EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
			SparseFloat.Value = static_cast<float>(i * 2);

			EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
		}

		const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
		AITEST_EQUAL(TEXT("Storage should have all SparseInt elements"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), static_cast<uint32>(Count));
		AITEST_EQUAL(TEXT("Storage should have all SparseFloat elements"),
			Storage.GetNumElementsOfType<FTestFragment_SparseFloat>(), static_cast<uint32>(Count));

		// Create entity collection for batch chunk destruction
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager, Entities,
			FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates, EntityCollections);

		// Destroy all entity chunks
		EntityManager->BatchDestroyEntityChunks(EntityCollections);

		// Verify all entities are invalid
		for (int i = 0; i < Count; ++i)
		{
			AITEST_FALSE(TEXT("Entity should be invalid after chunk destruction"),
				EntityManager->IsEntityValid(Entities[i]));
		}

		// Verify all sparse elements removed from storage
		AITEST_EQUAL(TEXT("Storage should have no SparseInt elements after chunk destruction"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), 0u);
		AITEST_EQUAL(TEXT("Storage should have no SparseFloat elements after chunk destruction"),
			Storage.GetNumElementsOfType<FTestFragment_SparseFloat>(), 0u);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchDestroyEntityChunks_WithSparseElements, "System.Mass.Stress.Commands.BatchDestroyChunks.WithSparseElements");


struct FCommands_BatchDestroyEntityChunks_PartialChunks : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
		const int32 Count = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 3.0f); // 3 full chunks

		TArray<FMassEntityHandle> AllEntities;
		EntityManager->BatchCreateEntities(IntsArchetype, Count, AllEntities);

		// Add sparse elements to all entities
		for (int i = 0; i < Count; ++i)
		{
			FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(AllEntities[i]);
			SparseInt.Value = i * 100;
		}

		const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();
		AITEST_EQUAL(TEXT("Storage should have all elements initially"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), static_cast<uint32>(Count));

		// Select only first 2 chunks worth of entities to destroy
		const int32 EntitiesToDestroy = EntitiesPerChunk * 2;
		TArray<FMassEntityHandle> EntitiesToDestroyArray;
		EntitiesToDestroyArray.Reserve(EntitiesToDestroy);
		for (int i = 0; i < EntitiesToDestroy; ++i)
		{
			EntitiesToDestroyArray.Add(AllEntities[i]);
		}

		// Create entity collection and destroy chunks
		TArray<FMassArchetypeEntityCollection> EntityCollections;
		UE::Mass::Utils::CreateEntityCollections(*EntityManager, EntitiesToDestroyArray,
			FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates, EntityCollections);

		EntityManager->BatchDestroyEntityChunks(EntityCollections);

		// Verify destroyed entities are invalid
		for (int i = 0; i < EntitiesToDestroy; ++i)
		{
			AITEST_FALSE(TEXT("Destroyed entity should be invalid"),
				EntityManager->IsEntityValid(AllEntities[i]));
		}

		// Verify remaining entities (third chunk) still valid with correct data
		const int32 ExpectedRemaining = Count - EntitiesToDestroy;
		AITEST_EQUAL(TEXT("Storage should have correct number of remaining elements"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), static_cast<uint32>(ExpectedRemaining));

		for (int i = EntitiesToDestroy; i < Count; ++i)
		{
			AITEST_TRUE(TEXT("Remaining entity should still be valid"),
				EntityManager->IsEntityValid(AllEntities[i]));
			AITEST_TRUE(TEXT("Remaining entity should still have sparse element"),
				EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(AllEntities[i]));

			FStructView SparseData = EntityManager->GetMutableSparseElementDataForEntity(
				FTestFragment_SparseInt::StaticStruct(), AllEntities[i]);
			AITEST_EQUAL(TEXT("Sparse data should be preserved for remaining entities"),
				SparseData.Get<FTestFragment_SparseInt>().Value, i * 100);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchDestroyEntityChunks_PartialChunks, "System.Mass.Stress.Commands.BatchDestroyChunks.PartialChunks");


struct FCommands_BatchDestroyEntities_MixedSparseTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 Count = 25;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsIntsArchetype, Count, Entities);

		// Add different combinations of sparse elements to different entities
		// First third: SparseInt + SparseFloat + SparseTag
		for (int i = 0; i < Count / 3; ++i)
		{
			EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
			EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
			EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
		}

		// Middle third: SparseInt + SparseTag only
		for (int i = Count / 3; i < (Count * 2) / 3; ++i)
		{
			EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entities[i]);
			EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entities[i]);
		}

		// Last third: SparseFloat only
		for (int i = (Count * 2) / 3; i < Count; ++i)
		{
			EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entities[i]);
		}

		const UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();

		// Verify initial counts
		constexpr uint32 InitialSparseIntCount = (Count / 3) + ((Count * 2) / 3 - Count / 3);
		constexpr uint32 InitialSparseFloatCount = (Count / 3) + (Count - (Count * 2) / 3);

		AITEST_EQUAL(TEXT("Initial SparseInt count should be correct"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), InitialSparseIntCount);
		AITEST_EQUAL(TEXT("Initial SparseFloat count should be correct"),
			Storage.GetNumElementsOfType<FTestFragment_SparseFloat>(), InitialSparseFloatCount);

		// Destroy all entities
		EntityManager->BatchDestroyEntities(Entities);

		// Verify all sparse elements removed regardless of type combinations
		AITEST_EQUAL(TEXT("All SparseInt elements should be removed"),
			Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), 0u);
		AITEST_EQUAL(TEXT("All SparseFloat elements should be removed"),
			Storage.GetNumElementsOfType<FTestFragment_SparseFloat>(), 0u);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FCommands_BatchDestroyEntities_MixedSparseTypes, "System.Mass.Stress.Commands.BatchDestroy.MixedSparseTypes");


#endif // WITH_MASSENTITY_DEBUG
} // UE::Mass::Test::Commands

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
