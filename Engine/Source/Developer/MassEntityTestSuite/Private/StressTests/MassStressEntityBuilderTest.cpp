// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityTestTypes.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"
#include "MassProcessingContext.h"


#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

#if 1

/**
 * Tests to be added:
 * - test observers triggering as expected, i.e. respecting the construction context
 * - entity grouping
 */

namespace UE::Mass::Test::EntityBuilder
{

//-----------------------------------------------------------------------------
// Sparse Elements Tests
//-----------------------------------------------------------------------------

struct FEntityBuilder_AddSparseFrag_Templated : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entity with sparse fragment using templated Add<>
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add<FTestFragment_SparseInt>(100)
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify sparse element was added to storage
		FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse element should exist in storage", SparseView.IsValid());

		const FTestFragment_SparseInt* SparseData = SparseView.GetPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Sparse data should be accessible", SparseData);
		AITEST_EQUAL("Sparse value should match initialization", SparseData->Value, 100);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_AddSparseFrag_Templated, "System.Mass.Stress.EntityBuilder.SparseElements.AddSparseFrag_Templated");

struct FEntityBuilder_AddSparseTag_Templated : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entity with sparse tag using templated Add<>
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add<FTestTag_SparseA>()
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify sparse tag was added
		const bool bHasTag = EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct());
		
		AITEST_TRUE("Entity should have sparse tag", bHasTag);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_AddSparseTag_Templated, "System.Mass.Stress.EntityBuilder.SparseElements.AddSparseTag_Templated");

struct FEntityBuilder_AddSparse_Mixed : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entity with mix of regular and sparse fragments
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add<FTestFragment_Int>(42)
			.Add<FTestFragment_SparseInt>(100)
			.Add<FTestFragment_Float>(3.14f)
			.Add<FTestFragment_SparseFloat>(2.718f)
			.Add<FTestTag_A>()
			.Add<FTestTag_SparseB>()
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify regular fragments
		const FTestFragment_Int& RegularInt = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
		AITEST_EQUAL("Regular int fragment should match", RegularInt.Value, 42);

		const FTestFragment_Float& RegularFloat = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(Entity);
		AITEST_EQUAL("Regular float fragment should match", RegularFloat.Value, 3.14f);

		// Verify regular tag
		AITEST_TRUE("Entity should have regular tag", EntityManager->DoesEntityHaveElement<FTestTag_A>(Entity));

		// Verify sparse fragments
		FConstStructView SparseIntView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse int should exist", SparseIntView.IsValid());
		AITEST_EQUAL("Sparse int value should match", SparseIntView.GetPtr<FTestFragment_SparseInt>()->Value, 100);

		FConstStructView SparseFloatView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseFloat::StaticStruct(), Entity);
		AITEST_TRUE("Sparse float should exist", SparseFloatView.IsValid());
		AITEST_EQUAL("Sparse float value should match", SparseFloatView.GetPtr<FTestFragment_SparseFloat>()->Value, 2.718f);

		// Verify sparse tag
		AITEST_TRUE("Entity should have sparse tag", EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_AddSparse_Mixed, "System.Mass.Stress.EntityBuilder.SparseElements.AddSparse_Mixed");

struct FEntityBuilder_AddSparse_InstancedStruct : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create sparse fragment instance
		FInstancedStruct SparseInstance = FInstancedStruct::Make<FTestFragment_SparseInt>(200);

		// Create entity using Add(FInstancedStruct)
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add(SparseInstance)
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify sparse element
		FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse element should exist", SparseView.IsValid());
		AITEST_EQUAL("Sparse value should match", SparseView.GetPtr<FTestFragment_SparseInt>()->Value, 200);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_AddSparse_InstancedStruct, "System.Mass.Stress.EntityBuilder.SparseElements.AddSparse_InstancedStruct");

struct FEntityBuilder_AddSparse_UScriptStruct : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Add sparse fragment by type pointer (no data initialization)
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add(FTestFragment_SparseInt::StaticStruct())
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify sparse element exists with default value
		FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse element should exist", SparseView.IsValid());
		AITEST_EQUAL("Sparse value should be default initialized (0)", SparseView.GetPtr<FTestFragment_SparseInt>()->Value, 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_AddSparse_UScriptStruct, "System.Mass.Stress.EntityBuilder.SparseElements.AddSparse_UScriptStruct");

struct FEntityBuilder_AddGetRef_Sparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Use Add_GetRef to get mutable reference and modify
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FTestFragment_SparseInt& SparseRef = Builder.Add_GetRef<FTestFragment_SparseInt>();
		SparseRef.Value = 500;

		FMassEntityHandle Entity = Builder.Commit();
		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify value was set through reference
		FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse element should exist", SparseView.IsValid());
		AITEST_EQUAL("Sparse value should match reference modification", SparseView.GetPtr<FTestFragment_SparseInt>()->Value, 500);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_AddGetRef_Sparse, "System.Mass.Stress.EntityBuilder.SparseElements.AddGetRef_Sparse");

struct FEntityBuilder_GetOrCreate_Sparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Use GetOrCreate twice - should return same instance
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FTestFragment_SparseInt& FirstRef = Builder.GetOrCreate<FTestFragment_SparseInt>(100);
		FirstRef.Value = 200;  // Modify

		FTestFragment_SparseInt& SecondRef = Builder.GetOrCreate<FTestFragment_SparseInt>(999);  // Should override
		AITEST_EQUAL("GetOrCreate should override existing value", SecondRef.Value, 999);

		FMassEntityHandle Entity = Builder.Commit();
		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify final value
		FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse element should exist", SparseView.IsValid());
		AITEST_EQUAL("Final sparse value should be from second GetOrCreate", SparseView.GetPtr<FTestFragment_SparseInt>()->Value, 999);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_GetOrCreate_Sparse, "System.Mass.Stress.EntityBuilder.SparseElements.GetOrCreate_Sparse");

struct FEntityBuilder_Find_Sparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create builder with sparse fragment
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		Builder.Add<FTestFragment_SparseInt>(300);

		// Find should return non-null pointer
		FTestFragment_SparseInt* Found = Builder.Find<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Find should return non-null for added sparse fragment", Found);
		AITEST_EQUAL("Found value should match", Found->Value, 300);

		// Find for non-added type should return null
		FTestFragment_SparseFloat* NotFound = Builder.Find<FTestFragment_SparseFloat>();
		AITEST_NULL("Find should return null for non-added sparse fragment", NotFound);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_Find_Sparse, "System.Mass.Stress.EntityBuilder.SparseElements.Find_Sparse");

struct FEntityBuilder_MultipleSparseFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entity with multiple sparse fragments
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add<FTestFragment_SparseInt>(111)
			.Add<FTestFragment_SparseFloat>(222.0f)
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify both sparse fragments
		FConstStructView SparseIntView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("First sparse element should exist", SparseIntView.IsValid());
		AITEST_EQUAL("First sparse value should match", SparseIntView.GetPtr<FTestFragment_SparseInt>()->Value, 111);

		FConstStructView SparseFloatView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseFloat::StaticStruct(), Entity);
		AITEST_TRUE("Second sparse element should exist", SparseFloatView.IsValid());
		AITEST_EQUAL("Second sparse value should match", SparseFloatView.GetPtr<FTestFragment_SparseFloat>()->Value, 222.0f);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_MultipleSparseFragments, "System.Mass.Stress.EntityBuilder.SparseElements.MultipleSparseFragments");

struct FEntityBuilder_MultipleSparseTags : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entity with multiple sparse tags
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add<FTestTag_SparseA>()
			.Add<FTestTag_SparseB>()
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify both sparse tags
		AITEST_TRUE("Entity should have first sparse tag", EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
		AITEST_TRUE("Entity should have second sparse tag", EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_MultipleSparseTags, "System.Mass.Stress.EntityBuilder.SparseElements.MultipleSparseTags");

struct FEntityBuilder_CopyDataFromEntity_WithSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create source entity with sparse elements
		UE::Mass::FEntityBuilder SourceBuilder(*EntityManager);
		FMassEntityHandle SourceEntity = SourceBuilder.Add<FTestFragment_Int>(42)
			.Add<FTestFragment_SparseInt>(999)
			.Add<FTestTag_SparseA>()
			.Commit();

		// Create new entity by copying from source
		UE::Mass::FEntityBuilder CopyBuilder(*EntityManager);
		const bool bSuccess = CopyBuilder.CopyDataFromEntity(SourceEntity);
		AITEST_TRUE("CopyDataFromEntity should succeed", bSuccess);

		FMassEntityHandle CopiedEntity = CopyBuilder.Commit();
		AITEST_TRUE("Copied entity should be valid", CopiedEntity.IsValid());

		// Verify regular fragment was copied
		const FTestFragment_Int& CopiedRegular = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(CopiedEntity);
		AITEST_EQUAL("Regular fragment should be copied", CopiedRegular.Value, 42);

		// Verify sparse fragment was copied
		FConstStructView CopiedSparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), CopiedEntity);
		AITEST_TRUE("Sparse fragment should be copied", CopiedSparseView.IsValid());
		AITEST_EQUAL("Sparse fragment value should match", CopiedSparseView.GetPtr<FTestFragment_SparseInt>()->Value, 999);

		// Verify sparse tag was copied
		AITEST_TRUE("Sparse tag should be copied", EntityManager->DoesEntityHaveElement(CopiedEntity, FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_CopyDataFromEntity_WithSparse, "System.Mass.Stress.EntityBuilder.SparseElements.CopyDataFromEntity_WithSparse");

struct FEntityBuilder_AppendDataFromEntity_WithSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create source entity with sparse elements
		UE::Mass::FEntityBuilder SourceBuilder(*EntityManager);
		FMassEntityHandle SourceEntity = SourceBuilder.Add<FTestFragment_SparseInt>(777)
			.Add<FTestTag_SparseA>()
			.Commit();

		// Create builder with some existing data
		UE::Mass::FEntityBuilder AppendBuilder(*EntityManager);
		AppendBuilder.Add<FTestFragment_Float>(1.5f)
			.Add<FTestTag_SparseB>();

		// Append data from source
		const bool bSuccess = AppendBuilder.AppendDataFromEntity(SourceEntity);
		AITEST_TRUE("AppendDataFromEntity should succeed", bSuccess);

		FMassEntityHandle ResultEntity = AppendBuilder.Commit();
		AITEST_TRUE("Result entity should be valid", ResultEntity.IsValid());

		// Verify original data is still there
		const FTestFragment_Float& OriginalFloat = EntityManager->GetFragmentDataChecked<FTestFragment_Float>(ResultEntity);
		AITEST_EQUAL("Original fragment should remain", OriginalFloat.Value, 1.5f);
		AITEST_TRUE("Original sparse tag should remain", EntityManager->DoesEntityHaveElement(ResultEntity, FTestTag_SparseB::StaticStruct()));

		// Verify appended sparse data
		FConstStructView AppendedSparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), ResultEntity);
		AITEST_TRUE("Appended sparse fragment should exist", AppendedSparseView.IsValid());
		AITEST_EQUAL("Appended sparse value should match", AppendedSparseView.GetPtr<FTestFragment_SparseInt>()->Value, 777);

		AITEST_TRUE("Appended sparse tag should exist", EntityManager->DoesEntityHaveElement(ResultEntity, FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_AppendDataFromEntity_WithSparse, "System.Mass.Stress.EntityBuilder.SparseElements.AppendDataFromEntity_WithSparse");

struct FEntityBuilder_Make_WithSparseFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Prepare composition and initial values
		FMassArchetypeCompositionDescriptor Composition;
		Composition.Add<FTestFragment_Int>();
		Composition.Add<FTestFragment_SparseInt>();
		Composition.Add<FTestTag_SparseA>();

		TArray<FInstancedStruct> InitialFragments;
		InitialFragments.Add(FInstancedStruct::Make<FTestFragment_Int>(50));
		InitialFragments.Add(FInstancedStruct::Make<FTestFragment_SparseInt>(888));

		// Create builder using Make with sparse fragments
		UE::Mass::FEntityBuilder Builder = UE::Mass::FEntityBuilder::Make(
			EntityManager->AsShared(),
			Composition,
			InitialFragments
		);

		FMassEntityHandle Entity = Builder.Commit();
		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify regular fragment
		const FTestFragment_Int& RegularInt = EntityManager->GetFragmentDataChecked<FTestFragment_Int>(Entity);
		AITEST_EQUAL("Regular fragment should match", RegularInt.Value, 50);

		// Verify sparse fragment
		FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse fragment should exist", SparseView.IsValid());
		AITEST_EQUAL("Sparse fragment value should match", SparseView.GetPtr<FTestFragment_SparseInt>()->Value, 888);

		// Verify sparse tag
		AITEST_TRUE("Sparse tag should exist", EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_Make_WithSparseFragments, "System.Mass.Stress.EntityBuilder.SparseElements.Make_WithSparseFragments");

struct FEntityBuilder_CommitAndReprepare_WithSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create builder with sparse elements
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		Builder.Add<FTestFragment_SparseInt>(111);

		// Commit first entity
		FMassEntityHandle Entity1 = Builder.CommitAndReprepare();
		AITEST_TRUE("First entity should be valid", Entity1.IsValid());

		// Modify and commit second entity
		FTestFragment_SparseInt& Ref = Builder.GetOrCreate<FTestFragment_SparseInt>(222);
		AITEST_EQUAL("Second entity data should be modified", Ref.Value, 222);

		FMassEntityHandle Entity2 = Builder.Commit();
		AITEST_TRUE("Second entity should be valid", Entity2.IsValid());
		AITEST_NOT_EQUAL("Entities should be different", Entity1.Index, Entity2.Index);

		// Verify both entities have correct values
		FConstStructView Sparse1 = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity1);
		AITEST_TRUE("The Sparse1 fragment view is expected to be valid", Sparse1.IsValid());
		AITEST_EQUAL("First entity should have original value", Sparse1.GetPtr<FTestFragment_SparseInt>()->Value, 111);

		FConstStructView Sparse2 = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity2);
		AITEST_TRUE("The Sparse2 fragment view is expected to be valid", Sparse2.IsValid());
		AITEST_EQUAL("Second entity should have modified value", Sparse2.GetPtr<FTestFragment_SparseInt>()->Value, 222);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_CommitAndReprepare_WithSparse, "System.Mass.Stress.EntityBuilder.SparseElements.CommitAndReprepare_WithSparse");

struct FEntityBuilder_DeferredCommit_WithSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create builder with deferred commit enabled
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		Builder.SetForceDeferredCommit(true);
		Builder.Add<FTestFragment_SparseInt>(333)
			.Add<FTestTag_SparseA>();

		FMassEntityHandle Entity = Builder.Commit();
		AITEST_TRUE("Entity should be valid after deferred commit", Entity.IsValid());

		// Flush commands
		EntityManager->FlushCommands();

		// Verify sparse elements were added through deferred path
		FConstStructView SparseView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse fragment should exist after flush", SparseView.IsValid());
		AITEST_EQUAL("Sparse value should match", SparseView.GetPtr<FTestFragment_SparseInt>()->Value, 333);

		AITEST_TRUE("Sparse tag should exist after flush", EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_DeferredCommit_WithSparse, "System.Mass.Stress.EntityBuilder.SparseElements.DeferredCommit_WithSparse");

struct FEntityBuilder_SparseOnly_NoRegularFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create entity with ONLY sparse elements (no regular fragments)
		UE::Mass::FEntityBuilder Builder(*EntityManager);
		FMassEntityHandle Entity = Builder.Add<FTestFragment_SparseInt>(444)
			.Add<FTestFragment_SparseFloat>(555.0f)
			.Add<FTestTag_SparseA>()
			.Add<FTestTag_SparseB>()
			.Commit();

		AITEST_TRUE("Entity should be valid", Entity.IsValid());

		// Verify all sparse elements exist
		FConstStructView SparseIntView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseInt::StaticStruct(), Entity);
		AITEST_TRUE("Sparse int should exist", SparseIntView.IsValid());
		AITEST_EQUAL("Sparse int value should match", SparseIntView.GetPtr<FTestFragment_SparseInt>()->Value, 444);

		FConstStructView SparseFloatView = EntityManager->GetSparseElementDataForEntity(FTestFragment_SparseFloat::StaticStruct(), Entity);
		AITEST_TRUE("Sparse float should exist", SparseFloatView.IsValid());
		AITEST_EQUAL("Sparse float value should match", SparseFloatView.GetPtr<FTestFragment_SparseFloat>()->Value, 555.0f);

		AITEST_TRUE("First sparse tag should exist", EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
		AITEST_TRUE("Second sparse tag should exist", EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityBuilder_SparseOnly_NoRegularFragments, "System.Mass.Stress.EntityBuilder.SparseElements.SparseOnly_NoRegularFragments");

} // UE::Mass::Test::EntityBuilder

#endif

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
