// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassEntityView.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::EntityView
{
#if WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// FMassEntityView Sparse Element Tests
//----------------------------------------------------------------------//

struct FEntityView_HasSparseFragment_SingleEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		// Test bug: HasFragment should check per-entity sparse element, not just archetype
		const FMassEntityHandle Entity1 = EntityManager->CreateEntity(IntsArchetype);
		const FMassEntityHandle Entity2 = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse fragment to only Entity1
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity1);

		FMassEntityView View1(*EntityManager, Entity1);
		FMassEntityView View2(*EntityManager, Entity2);

		// Both are in same archetype, but only Entity1 has the sparse fragment
		AITEST_EQUAL(TEXT("Both entities in same archetype"),
			EntityManager->GetArchetypeForEntity(Entity1),
			EntityManager->GetArchetypeForEntity(Entity2));

		AITEST_TRUE(TEXT("Entity1 should have sparse fragment"),
			View1.HasElement<FTestFragment_SparseInt>());
		AITEST_FALSE(TEXT("Entity2 should NOT have sparse fragment"),
			View2.HasElement<FTestFragment_SparseInt>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_HasSparseFragment_SingleEntity, "System.Mass.Stress.EntityView.Sparse.HasFragment.PerEntity");


struct FEntityView_HasSparseTag_SingleEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity1 = EntityManager->CreateEntity(FloatsArchetype);
		const FMassEntityHandle Entity2 = EntityManager->CreateEntity(FloatsArchetype);

		// Add sparse tag to only Entity1
		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entity1);

		FMassEntityView View1(*EntityManager, Entity1);
		FMassEntityView View2(*EntityManager, Entity2);

		AITEST_TRUE(TEXT("Entity1 should have sparse tag"),
			View1.HasElement<FTestTag_SparseA>());
		AITEST_FALSE(TEXT("Entity2 should NOT have sparse tag"),
			View2.HasElement<FTestTag_SparseA>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_HasSparseTag_SingleEntity, "System.Mass.Stress.EntityView.Sparse.HasTag.PerEntity");


struct FEntityView_HasElement_BugTest : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		// BUG: HasElement doesn't check per-entity sparse elements correctly
		const FMassEntityHandle Entity1 = EntityManager->CreateEntity(IntsArchetype);
		const FMassEntityHandle Entity2 = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse fragment to only Entity1
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity1);

		FMassEntityView View1(*EntityManager, Entity1);
		FMassEntityView View2(*EntityManager, Entity2);

		// HasElement with bIncludeSparse=true should check per-entity
		AITEST_TRUE(TEXT("Entity1.HasElement should return true for sparse fragment"),
			View1.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));
		AITEST_FALSE(TEXT("Entity2.HasElement should return false for sparse fragment"),
			View2.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));

		// HasElement with bIncludeSparse=false should only check archetype
		AITEST_FALSE(TEXT("Entity1.HasElement(false) should return false"),
			View1.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::No));
		AITEST_FALSE(TEXT("Entity2.HasElement(false) should return false"),
			View2.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::No));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_HasElement_BugTest, "System.Mass.Stress.EntityView.Sparse.HasElement.BugTest");


struct FEntityView_HasElement_SparseTag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity1 = EntityManager->CreateEntity(FloatsArchetype);
		const FMassEntityHandle Entity2 = EntityManager->CreateEntity(FloatsArchetype);

		// Add sparse tag to only Entity1
		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entity1);

		FMassEntityView View1(*EntityManager, Entity1);
		FMassEntityView View2(*EntityManager, Entity2);

		// HasElement should work with sparse tags too
		AITEST_TRUE(TEXT("Entity1 HasElement should detect sparse tag"),
			View1.HasElement<FTestTag_SparseA>(UE::Mass::EIncludeSparseElements::Yes));
		AITEST_FALSE(TEXT("Entity2 HasElement should not detect sparse tag"),
			View2.HasElement<FTestTag_SparseA>(UE::Mass::EIncludeSparseElements::Yes));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_HasElement_SparseTag, "System.Mass.Stress.EntityView.Sparse.HasElement.Tag");


struct FEntityView_GetSparseFragmentDataPtr : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse fragment and set value
		FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
		SparseData.Value = 42;

		FMassEntityView EntityView(*EntityManager, Entity);

		// Test const version
		const FTestFragment_SparseInt* ConstFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL(TEXT("GetSparseFragmentDataPtr (const) should return valid pointer"), ConstFragment);
		AITEST_EQUAL(TEXT("Value should be accessible"), ConstFragment->Value, 42);

		// Test mutable version
		FTestFragment_SparseInt* MutableFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL(TEXT("GetSparseFragmentDataPtr (mutable) should return valid pointer"), MutableFragment);

		// Modify value
		MutableFragment->Value = 100;

		// Verify modification persisted
		const FTestFragment_SparseInt* VerifyFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_EQUAL(TEXT("Modified value should persist"), VerifyFragment->Value, 100);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_GetSparseFragmentDataPtr, "System.Mass.Stress.EntityView.Sparse.GetSparseFragmentDataPtr");


struct FEntityView_GetSparseFragmentDataPtr_NonExistent : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		FMassEntityView EntityView(*EntityManager, Entity);

		// Get pointer to sparse fragment that doesn't exist
		const FTestFragment_SparseFloat* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();
		AITEST_NULL(TEXT("GetSparseFragmentDataPtr should return null for non-existent fragment"), Fragment);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_GetSparseFragmentDataPtr_NonExistent, "System.Mass.Stress.EntityView.Sparse.GetSparseFragmentDataPtr.NonExistent");


struct FEntityView_SparseFragments_MultipleTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype);

		// Add multiple sparse types
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entity);
		EntityManager->AddSparseElementToEntity<FTestTag_SparseA>(Entity);

		FMassEntityView EntityView(*EntityManager, Entity);

		// Verify HasElement works for all
		AITEST_TRUE(TEXT("HasElement detects sparse int"),
			EntityView.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));
		AITEST_TRUE(TEXT("HasElement detects sparse float"),
			EntityView.HasElement<FTestFragment_SparseFloat>(UE::Mass::EIncludeSparseElements::Yes));
		AITEST_TRUE(TEXT("HasElement detects sparse tag"),
			EntityView.HasElement<FTestTag_SparseA>(UE::Mass::EIncludeSparseElements::Yes));

		// Verify GetSparseFragmentDataPtr works for all fragment types
		AITEST_NOT_NULL(TEXT("Can get sparse int pointer"),
			EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>());
		AITEST_NOT_NULL(TEXT("Can get sparse float pointer"),
			EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_SparseFragments_MultipleTypes, "System.Mass.Stress.EntityView.Sparse.MultipleTypes");


struct FEntityView_SparseFragment_AfterArchetypeChange : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse fragment
		FTestFragment_SparseInt& SparseData = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
		SparseData.Value = 777;

		// Get initial view
		FMassEntityView View1(*EntityManager, Entity);
		AITEST_TRUE(TEXT("View1 detects sparse fragment"),
			View1.HasElement<FTestFragment_SparseInt>());

		// Change archetype by adding regular fragment
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_Float::StaticStruct());

		// Old view should be invalidated
		AITEST_FALSE(TEXT("Old view invalidated after archetype change"), View1.IsValid());

		// Create new view
		FMassEntityView View2(*EntityManager, Entity);
		AITEST_TRUE(TEXT("New view is valid"), View2.IsValid());

		// Verify sparse fragment still accessible
		AITEST_TRUE(TEXT("Sparse fragment persisted through archetype change"),
			View2.HasElement<FTestFragment_SparseInt>());

		const FTestFragment_SparseInt* Fragment = View2.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL(TEXT("Can still get sparse fragment pointer"), Fragment);
		AITEST_EQUAL(TEXT("Sparse fragment data preserved"), Fragment->Value, 777);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_SparseFragment_AfterArchetypeChange, "System.Mass.Stress.EntityView.Sparse.AfterArchetypeChange");


struct FEntityView_SparseFragment_RemoveAndCheck : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse fragment
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);

		FMassEntityView EntityView(*EntityManager, Entity);
		AITEST_TRUE(TEXT("Fragment exists before removal"),
			EntityView.HasElement<FTestFragment_SparseInt>());

		// Remove sparse fragment
		EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Note: EntityView doesn't get invalidated by sparse element changes (only archetype changes)
		AITEST_TRUE(TEXT("View still valid after sparse removal"), EntityView.IsValid());

		// But HasElement should return false now
		AITEST_FALSE(TEXT("HasElement returns false after removal"),
			EntityView.HasElement<FTestFragment_SparseInt>());

		// And pointer should be null
		AITEST_NULL(TEXT("GetSparseFragmentDataPtr returns null after removal"),
			EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_SparseFragment_RemoveAndCheck, "System.Mass.Stress.EntityView.Sparse.RemoveAndCheck");


struct FEntityView_HasElement_bIncludeSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Entity has regular Int fragment (from archetype) but not Float
		AITEST_TRUE(TEXT("Entity has Int fragment in archetype"),
			EntityManager->GetFragmentDataPtr<FTestFragment_Int>(Entity) != nullptr);
		AITEST_FALSE(TEXT("Entity doesn't have Float fragment in archetype"),
			EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity) != nullptr);

		// Add sparse fragment
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseFloat>(Entity);

		FMassEntityView EntityView(*EntityManager, Entity);

		// HasElement with bIncludeSparse=true should detect sparse fragment
		AITEST_TRUE(TEXT("HasElement(true) detects sparse fragment"),
			EntityView.HasElement<FTestFragment_SparseFloat>(UE::Mass::EIncludeSparseElements::Yes));

		// HasElement with bIncludeSparse=false should NOT detect sparse fragment
		AITEST_FALSE(TEXT("HasElement(false) doesn't detect sparse fragment"),
			EntityView.HasElement<FTestFragment_SparseFloat>(UE::Mass::EIncludeSparseElements::No));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_HasElement_bIncludeSparse, "System.Mass.Stress.EntityView.Sparse.HasElement.bIncludeSparse");


struct FEntityView_SparseAcrossDifferentArchetypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle EntityInts = EntityManager->CreateEntity(IntsArchetype);
		const FMassEntityHandle EntityFloats = EntityManager->CreateEntity(FloatsArchetype);

		// Add same sparse fragment to entities in different archetypes
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(EntityInts);
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(EntityFloats);

		FMassEntityView ViewInts(*EntityManager, EntityInts);
		FMassEntityView ViewFloats(*EntityManager, EntityFloats);

		// Both should detect the sparse fragment
		AITEST_TRUE(TEXT("Ints archetype entity has sparse fragment"),
			ViewInts.HasElement<FTestFragment_SparseInt>());
		AITEST_TRUE(TEXT("Floats archetype entity has sparse fragment"),
			ViewFloats.HasElement<FTestFragment_SparseInt>());

		// Both should provide access to the fragment
		AITEST_NOT_NULL(TEXT("Ints entity can get sparse fragment"),
			ViewInts.GetSparseFragmentDataPtr<FTestFragment_SparseInt>());
		AITEST_NOT_NULL(TEXT("Floats entity can get sparse fragment"),
			ViewFloats.GetSparseFragmentDataPtr<FTestFragment_SparseInt>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_SparseAcrossDifferentArchetypes, "System.Mass.Stress.EntityView.Sparse.AcrossDifferentArchetypes");


struct FEntityView_MixedRegularAndSparseFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsIntsArchetype);

		// Entity has regular Float and Int fragments from archetype
		// Add sparse fragment
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);

		FMassEntityView EntityView(*EntityManager, Entity);

		// Regular fragments should be accessible
		FTestFragment_Float* FloatFragment = EntityView.GetFragmentDataPtr<FTestFragment_Float>();
		FTestFragment_Int* IntFragment = EntityView.GetFragmentDataPtr<FTestFragment_Int>();
		AITEST_NOT_NULL(TEXT("Can access regular Float fragment"), FloatFragment);
		AITEST_NOT_NULL(TEXT("Can access regular Int fragment"), IntFragment);

		// Sparse fragment should be accessible
		FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL(TEXT("Can access sparse fragment"), SparseFragment);

		// HasFragment should detect both types
		AITEST_TRUE(TEXT("HasFragment detects regular Float"),
			EntityView.HasFragment<FTestFragment_Float>());
		AITEST_TRUE(TEXT("HasFragment detects regular Int"),
			EntityView.HasFragment<FTestFragment_Int>());
		AITEST_TRUE(TEXT("HasElement detects sparse Int"),
			EntityView.HasElement<FTestFragment_SparseInt>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_MixedRegularAndSparseFragments, "System.Mass.Stress.EntityView.Sparse.MixedRegularAndSparse");


struct FEntityView_TryMakeView_WithSparseElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle ValidEntity = EntityManager->CreateEntity(IntsArchetype);
		EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(ValidEntity);

		const FMassEntityHandle InvalidEntity; // Default constructor = invalid

		// TryMakeView with valid entity
		FMassEntityView ValidView = FMassEntityView::TryMakeView(*EntityManager, ValidEntity);
		AITEST_TRUE(TEXT("TryMakeView succeeds for valid entity"), ValidView.IsSet());
		AITEST_TRUE(TEXT("View detects sparse fragment"),
			ValidView.HasElement<FTestFragment_SparseInt>());

		// TryMakeView with invalid entity
		FMassEntityView InvalidView = FMassEntityView::TryMakeView(*EntityManager, InvalidEntity);
		AITEST_FALSE(TEXT("TryMakeView returns unset view for invalid entity"), InvalidView.IsSet());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FEntityView_TryMakeView_WithSparseElements, "System.Mass.Stress.EntityView.Sparse.TryMakeView");

#endif // WITH_MASSENTITY_DEBUG
} // UE::Mass::Test::EntityView

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
