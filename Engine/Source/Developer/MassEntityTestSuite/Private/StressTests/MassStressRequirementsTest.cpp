// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTestTypes.h"
#include "MassRequirements.h"
#include "MassArchetypeTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::Mass::Test::Requirements
{
#if WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// FMassFragmentRequirements - Fragment Requirements Tests
//----------------------------------------------------------------------//

struct FRequirements_AddRequirement_All : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Add fragment with All presence
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

		// Verify fragment is in RequiredAllFragments
		AITEST_TRUE("RequiredAllFragments should contain the added fragment",
			Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredAnyFragments should not contain the fragment",
			Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredOptionalFragments should not contain the fragment",
			Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredNoneFragments should not contain the fragment",
			Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());

		// Verify fragment requirements description
		TConstArrayView<FMassFragmentRequirementDescription> FragmentRequirements = Requirements.GetFragmentRequirements();
		AITEST_EQUAL("Should have one fragment requirement", FragmentRequirements.Num(), 1);
		AITEST_EQUAL("Fragment type should match", FragmentRequirements[0].StructType, FTestFragment_Int::StaticStruct());
		AITEST_EQUAL("Access mode should be ReadOnly", FragmentRequirements[0].AccessMode, EMassFragmentAccess::ReadOnly);
		AITEST_EQUAL("Presence should be All", FragmentRequirements[0].Presence, EMassFragmentPresence::All);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_All, "System.Mass.Stress.Requirements.Fragment.AddRequirement.All");


struct FRequirements_AddRequirement_Any : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);

		AITEST_FALSE("RequiredAllFragments should not contain the fragment",
			Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_TRUE("RequiredAnyFragments should contain the added fragment",
			Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredOptionalFragments should not contain the fragment",
			Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredNoneFragments should not contain the fragment",
			Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_Any, "System.Mass.Stress.Requirements.Fragment.AddRequirement.Any");


struct FRequirements_AddRequirement_Optional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);

		AITEST_FALSE("RequiredAllFragments should not contain the fragment",
			Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredAnyFragments should not contain the fragment",
			Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
		AITEST_TRUE("RequiredOptionalFragments should contain the added fragment",
			Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredNoneFragments should not contain the fragment",
			Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_Optional, "System.Mass.Stress.Requirements.Fragment.AddRequirement.Optional");


struct FRequirements_AddRequirement_None : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);

		AITEST_FALSE("RequiredAllFragments should not contain the fragment",
			Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredAnyFragments should not contain the fragment",
			Requirements.GetRequiredAnyFragments().Contains<FTestFragment_Int>());
		AITEST_FALSE("RequiredOptionalFragments should not contain the fragment",
			Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Int>());
		AITEST_TRUE("RequiredNoneFragments should contain the added fragment",
			Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Int>());

		// None presence should not add to FragmentRequirements array
		AITEST_EQUAL("FragmentRequirements should be empty for None presence",
			Requirements.GetFragmentRequirements().Num(), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_None, "System.Mass.Stress.Requirements.Fragment.AddRequirement.None");


struct FRequirements_AddRequirement_NonTemplate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use non-template version
		Requirements.AddRequirement(FTestFragment_Float::StaticStruct(), EMassFragmentAccess::ReadWrite, EMassFragmentPresence::All);

		AITEST_TRUE("RequiredAllFragments should contain the added fragment",
			Requirements.GetRequiredAllFragments().Contains(FTestFragment_Float::StaticStruct()));

		TConstArrayView<FMassFragmentRequirementDescription> FragmentRequirements = Requirements.GetFragmentRequirements();
		AITEST_EQUAL("Should have one fragment requirement", FragmentRequirements.Num(), 1);
		AITEST_EQUAL("Access mode should be ReadWrite", FragmentRequirements[0].AccessMode, EMassFragmentAccess::ReadWrite);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_NonTemplate, "System.Mass.Stress.Requirements.Fragment.AddRequirement.NonTemplate");


struct FRequirements_AddRequirement_Multiple : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Requirements.AddRequirement<FTestFragment_Bool>(EMassFragmentAccess::None, EMassFragmentPresence::None);

		AITEST_TRUE("RequiredAllFragments should contain Int",
			Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_TRUE("RequiredOptionalFragments should contain Float",
			Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
		AITEST_TRUE("RequiredNoneFragments should contain Bool",
			Requirements.GetRequiredNoneFragments().Contains<FTestFragment_Bool>());

		// FragmentRequirements should have 2 entries (All and Optional, not None)
		AITEST_EQUAL("Should have two fragment requirements (excluding None)",
			Requirements.GetFragmentRequirements().Num(), 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_Multiple, "System.Mass.Stress.Requirements.Fragment.AddRequirement.Multiple");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Tag Requirements Tests
//----------------------------------------------------------------------//

struct FRequirements_AddTagRequirement_All : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);

		AITEST_TRUE("RequiredAllTags should contain the added tag",
			Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredAnyTags should not contain the tag",
			Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredNoneTags should not contain the tag",
			Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredOptionalTags should not contain the tag",
			Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_All, "System.Mass.Stress.Requirements.Tag.AddTagRequirement.All");


struct FRequirements_AddTagRequirement_Any : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Any);

		AITEST_FALSE("RequiredAllTags should not contain the tag",
			Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_TRUE("RequiredAnyTags should contain the added tag",
			Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredNoneTags should not contain the tag",
			Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredOptionalTags should not contain the tag",
			Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_Any, "System.Mass.Stress.Requirements.Tag.AddTagRequirement.Any");


struct FRequirements_AddTagRequirement_None : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::None);

		AITEST_FALSE("RequiredAllTags should not contain the tag",
			Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredAnyTags should not contain the tag",
			Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
		AITEST_TRUE("RequiredNoneTags should contain the added tag",
			Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredOptionalTags should not contain the tag",
			Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_None, "System.Mass.Stress.Requirements.Tag.AddTagRequirement.None");


struct FRequirements_AddTagRequirement_Optional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Optional);

		AITEST_FALSE("RequiredAllTags should not contain the tag",
			Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredAnyTags should not contain the tag",
			Requirements.GetRequiredAnyTags().Contains<FTestTag_A>());
		AITEST_FALSE("RequiredNoneTags should not contain the tag",
			Requirements.GetRequiredNoneTags().Contains<FTestTag_A>());
		AITEST_TRUE("RequiredOptionalTags should contain the added tag",
			Requirements.GetRequiredOptionalTags().Contains<FTestTag_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_Optional, "System.Mass.Stress.Requirements.Tag.AddTagRequirement.Optional");


struct FRequirements_AddTagRequirement_NonTemplate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddTagRequirement(FTestTag_B::StaticStruct(), EMassFragmentPresence::All);

		AITEST_TRUE("RequiredAllTags should contain the added tag",
			Requirements.GetRequiredAllTags().Contains(FTestTag_B::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_NonTemplate, "System.Mass.Stress.Requirements.Tag.AddTagRequirement.NonTemplate");


struct FRequirements_AddTagRequirements_Batch : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		FMassTagBitSet TagBitSet;
		TagBitSet.Add<FTestTag_A>();
		TagBitSet.Add<FTestTag_B>();

		// Add batch of tags with All presence
		Requirements.AddTagRequirements<EMassFragmentPresence::All>(TagBitSet);

		AITEST_TRUE("RequiredAllTags should contain Tag A",
			Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_TRUE("RequiredAllTags should contain Tag B",
			Requirements.GetRequiredAllTags().Contains<FTestTag_B>());

		// Add another batch with None presence
		FMassTagBitSet NoneTagBitSet;
		NoneTagBitSet.Add<FTestTag_C>();
		Requirements.AddTagRequirements<EMassFragmentPresence::None>(NoneTagBitSet);

		AITEST_TRUE("RequiredNoneTags should contain Tag C",
			Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirements_Batch, "System.Mass.Stress.Requirements.Tag.AddTagRequirements.Batch");


struct FRequirements_ClearTagRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Add tags to various categories
		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::Any);
		Requirements.AddTagRequirement<FTestTag_C>(EMassFragmentPresence::None);

		// Verify tags are present
		AITEST_TRUE("Tag A should be in AllTags", Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_TRUE("Tag B should be in AnyTags", Requirements.GetRequiredAnyTags().Contains<FTestTag_B>());
		AITEST_TRUE("Tag C should be in NoneTags", Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());

		// Clear Tag A and B
		FMassTagBitSet TagsToClear;
		TagsToClear.Add<FTestTag_A>();
		TagsToClear.Add<FTestTag_B>();
		Requirements.ClearTagRequirements(TagsToClear);

		// Verify cleared tags are gone
		AITEST_FALSE("Tag A should be cleared from AllTags", Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_FALSE("Tag B should be cleared from AnyTags", Requirements.GetRequiredAnyTags().Contains<FTestTag_B>());

		// Tag C should still be present (wasn't in clear set)
		AITEST_TRUE("Tag C should still be in NoneTags", Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_ClearTagRequirements, "System.Mass.Stress.Requirements.Tag.ClearTagRequirements");


struct FRequirements_AddTagRequirement_Multiple : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::Any);
		Requirements.AddTagRequirement<FTestTag_C>(EMassFragmentPresence::None);
		Requirements.AddTagRequirement<FTestTag_D>(EMassFragmentPresence::Optional);

		AITEST_TRUE("Tag A in AllTags", Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_TRUE("Tag B in AnyTags", Requirements.GetRequiredAnyTags().Contains<FTestTag_B>());
		AITEST_TRUE("Tag C in NoneTags", Requirements.GetRequiredNoneTags().Contains<FTestTag_C>());
		AITEST_TRUE("Tag D in OptionalTags", Requirements.GetRequiredOptionalTags().Contains<FTestTag_D>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_Multiple, "System.Mass.Stress.Requirements.Tag.AddTagRequirement.Multiple");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Sparse Element Requirements Tests
// These tests use the unified API (AddRequirement/AddTagRequirement) with sparse elements
// to verify that the functions automatically detect and route sparse elements correctly
//----------------------------------------------------------------------//

struct FRequirements_AddRequirement_SparseFragment_All : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddRequirement with a sparse fragment - should route to sparse storage
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

		// Should be in sparse elements, NOT regular fragments
		AITEST_TRUE("RequiredAllSparseElements should contain the sparse fragment",
			Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("RequiredAllFragments should NOT contain the sparse fragment",
			Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("RequiredAnySparseElements should not contain the sparse fragment",
			Requirements.GetRequiredAnySparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("RequiredOptionalSparseElements should not contain the sparse fragment",
			Requirements.GetRequiredOptionalSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("RequiredNoneSparseElements should not contain the sparse fragment",
			Requirements.GetRequiredNoneSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));

		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
		AITEST_TRUE("HasSparseRequirements should return true", Requirements.HasSparseRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_SparseFragment_All, "System.Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.All");


struct FRequirements_AddRequirement_SparseFragment_Any : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddRequirement with a sparse fragment - should route to sparse storage
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Any);

		AITEST_FALSE("RequiredAllSparseElements should not contain the sparse fragment",
			Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("RequiredAnySparseElements should contain the sparse fragment",
			Requirements.GetRequiredAnySparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("RequiredAnyFragments should NOT contain the sparse fragment",
			Requirements.GetRequiredAnyFragments().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_SparseFragment_Any, "System.Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.Any");


struct FRequirements_AddRequirement_SparseFragment_Optional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddRequirement with a sparse fragment - should route to sparse storage
		Requirements.AddRequirement<FTestFragment_SparseFloat>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);

		AITEST_FALSE("RequiredAllSparseElements should not contain the sparse fragment",
			Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_TRUE("RequiredOptionalSparseElements should contain the sparse fragment",
			Requirements.GetRequiredOptionalSparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_FALSE("RequiredOptionalFragments should NOT contain the sparse fragment",
			Requirements.GetRequiredOptionalFragments().Contains(FTestFragment_SparseFloat::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_SparseFragment_Optional, "System.Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.Optional");


struct FRequirements_AddRequirement_SparseFragment_None : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddRequirement with a sparse fragment - should route to sparse storage
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::None, EMassFragmentPresence::None);

		AITEST_TRUE("RequiredNoneSparseElements should contain the sparse fragment",
			Requirements.GetRequiredNoneSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("RequiredNoneFragments should NOT contain the sparse fragment",
			Requirements.GetRequiredNoneFragments().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_SparseFragment_None, "System.Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.None");


struct FRequirements_AddTagRequirement_SparseTag_All : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddTagRequirement with a sparse tag - should route to sparse storage
		Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::All);

		AITEST_TRUE("RequiredAllSparseElements should contain the sparse tag",
			Requirements.GetRequiredAllSparseElements().Contains(FTestTag_SparseA::StaticStruct()));
		AITEST_FALSE("RequiredAllTags should NOT contain the sparse tag",
			Requirements.GetRequiredAllTags().Contains(FTestTag_SparseA::StaticStruct()));

		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
		AITEST_TRUE("HasSparseRequirements should return true", Requirements.HasSparseRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_SparseTag_All, "System.Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.All");


struct FRequirements_AddTagRequirement_SparseTag_Any : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddTagRequirement with a sparse tag - should route to sparse storage
		Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::Any);

		AITEST_TRUE("RequiredAnySparseElements should contain the sparse tag",
			Requirements.GetRequiredAnySparseElements().Contains(FTestTag_SparseA::StaticStruct()));
		AITEST_FALSE("RequiredAnyTags should NOT contain the sparse tag",
			Requirements.GetRequiredAnyTags().Contains(FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_SparseTag_Any, "System.Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.Any");


struct FRequirements_AddTagRequirement_SparseTag_Optional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddTagRequirement with a sparse tag - should route to sparse storage
		Requirements.AddTagRequirement<FTestTag_SparseB>(EMassFragmentPresence::Optional);

		AITEST_TRUE("RequiredOptionalSparseElements should contain the sparse tag",
			Requirements.GetRequiredOptionalSparseElements().Contains(FTestTag_SparseB::StaticStruct()));
		AITEST_FALSE("RequiredOptionalTags should NOT contain the sparse tag",
			Requirements.GetRequiredOptionalTags().Contains(FTestTag_SparseB::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_SparseTag_Optional, "System.Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.Optional");


struct FRequirements_AddTagRequirement_SparseTag_None : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use AddTagRequirement with a sparse tag - should route to sparse storage
		Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::None);

		AITEST_TRUE("RequiredNoneSparseElements should contain the sparse tag",
			Requirements.GetRequiredNoneSparseElements().Contains(FTestTag_SparseA::StaticStruct()));
		AITEST_FALSE("RequiredNoneTags should NOT contain the sparse tag",
			Requirements.GetRequiredNoneTags().Contains(FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_SparseTag_None, "System.Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.None");


struct FRequirements_AddRequirement_SparseFragment_NonTemplate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use non-template AddRequirement with sparse fragment
		Requirements.AddRequirement(FTestFragment_SparseInt::StaticStruct(), EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);

		AITEST_TRUE("RequiredAllSparseElements should contain the sparse fragment",
			Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("RequiredAllFragments should NOT contain the sparse fragment",
			Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddRequirement_SparseFragment_NonTemplate, "System.Mass.Stress.Requirements.Sparse.AddRequirement.SparseFragment.NonTemplate");


struct FRequirements_AddTagRequirement_SparseTag_NonTemplate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Use non-template AddTagRequirement with sparse tag
		Requirements.AddTagRequirement(FTestTag_SparseB::StaticStruct(), EMassFragmentPresence::Any);

		AITEST_TRUE("RequiredAnySparseElements should contain the sparse tag",
			Requirements.GetRequiredAnySparseElements().Contains(FTestTag_SparseB::StaticStruct()));
		AITEST_FALSE("RequiredAnyTags should NOT contain the sparse tag",
			Requirements.GetRequiredAnyTags().Contains(FTestTag_SparseB::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddTagRequirement_SparseTag_NonTemplate, "System.Mass.Stress.Requirements.Sparse.AddTagRequirement.SparseTag.NonTemplate");


struct FRequirements_UnifiedAPI_MixedSparseElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Mix of sparse fragments and sparse tags using unified API
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_SparseFloat>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::Any);
		Requirements.AddTagRequirement<FTestTag_SparseB>(EMassFragmentPresence::None);

		// All should be routed to sparse storage
		AITEST_TRUE("Sparse int in AllSparseElements",
			Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Sparse float in OptionalSparseElements",
			Requirements.GetRequiredOptionalSparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_TRUE("Sparse tag A in AnySparseElements",
			Requirements.GetRequiredAnySparseElements().Contains(FTestTag_SparseA::StaticStruct()));
		AITEST_TRUE("Sparse tag B in NoneSparseElements",
			Requirements.GetRequiredNoneSparseElements().Contains(FTestTag_SparseB::StaticStruct()));

		// None should be in regular storage
		AITEST_FALSE("Sparse int NOT in AllFragments",
			Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("Sparse float NOT in OptionalFragments",
			Requirements.GetRequiredOptionalFragments().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_FALSE("Sparse tag A NOT in AnyTags",
			Requirements.GetRequiredAnyTags().Contains(FTestTag_SparseA::StaticStruct()));
		AITEST_FALSE("Sparse tag B NOT in NoneTags",
			Requirements.GetRequiredNoneTags().Contains(FTestTag_SparseB::StaticStruct()));

		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
		AITEST_TRUE("HasSparseRequirements should return true", Requirements.HasSparseRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_UnifiedAPI_MixedSparseElements, "System.Mass.Stress.Requirements.Sparse.UnifiedAPI.Mixed");


struct FRequirements_UnifiedAPI_MixedRegularAndSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Add regular fragments
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		// Add regular tags
		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::None);
		// Add sparse fragments using unified API
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_SparseFloat>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Any);
		// Add sparse tags using unified API
		Requirements.AddTagRequirement<FTestTag_SparseA>(EMassFragmentPresence::Optional);
		Requirements.AddTagRequirement<FTestTag_SparseB>(EMassFragmentPresence::None);

		// Regular elements should be in regular storage
		AITEST_TRUE("Int fragment in AllFragments", Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_TRUE("Float fragment in OptionalFragments", Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
		AITEST_TRUE("Tag A in AllTags", Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_TRUE("Tag B in NoneTags", Requirements.GetRequiredNoneTags().Contains<FTestTag_B>());

		// Sparse elements should be in sparse storage
		AITEST_TRUE("Sparse int in AllSparseElements",
			Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Sparse float in AnySparseElements",
			Requirements.GetRequiredAnySparseElements().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_TRUE("Sparse tag A in OptionalSparseElements",
			Requirements.GetRequiredOptionalSparseElements().Contains(FTestTag_SparseA::StaticStruct()));
		AITEST_TRUE("Sparse tag B in NoneSparseElements",
			Requirements.GetRequiredNoneSparseElements().Contains(FTestTag_SparseB::StaticStruct()));

		// Sparse elements should NOT be in regular storage
		AITEST_FALSE("Sparse int NOT in regular AllFragments",
			Requirements.GetRequiredAllFragments().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("Sparse float NOT in regular AnyFragments",
			Requirements.GetRequiredAnyFragments().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_FALSE("Sparse tag A NOT in regular OptionalTags",
			Requirements.GetRequiredOptionalTags().Contains(FTestTag_SparseA::StaticStruct()));
		AITEST_FALSE("Sparse tag B NOT in regular NoneTags",
			Requirements.GetRequiredNoneTags().Contains(FTestTag_SparseB::StaticStruct()));

		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties
		AITEST_TRUE("HasSparseRequirements should return true", Requirements.HasSparseRequirements());
		AITEST_TRUE("HasPositiveRequirements should return true", Requirements.HasPositiveRequirements());
		AITEST_TRUE("HasNegativeRequirements should return true", Requirements.HasNegativeRequirements());
		AITEST_TRUE("HasOptionalRequirements should return true", Requirements.HasOptionalRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_UnifiedAPI_MixedRegularAndSparse, "System.Mass.Stress.Requirements.Sparse.UnifiedAPI.MixedRegularAndSparse");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - State and Validation Tests
//----------------------------------------------------------------------//

struct FRequirements_IsEmpty : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		AITEST_TRUE("Newly created requirements should be empty", Requirements.IsEmpty());

		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_FALSE("Requirements with fragment should not be empty", Requirements.IsEmpty());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_IsEmpty, "System.Mass.Stress.Requirements.State.IsEmpty");


struct FRequirements_HasPositiveRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Initially no positive requirements
		AITEST_FALSE("Empty requirements have no positive requirements", Requirements.HasPositiveRequirements());

		// Add "All" fragment - this is a positive requirement
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Requirements with All fragment have positive requirements", Requirements.HasPositiveRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_HasPositiveRequirements, "System.Mass.Stress.Requirements.State.HasPositiveRequirements");


struct FRequirements_HasNegativeRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		AITEST_FALSE("Empty requirements have no negative requirements", Requirements.HasNegativeRequirements());

		// Add "None" fragment - this is a negative requirement
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Requirements with None fragment have negative requirements", Requirements.HasNegativeRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_HasNegativeRequirements, "System.Mass.Stress.Requirements.State.HasNegativeRequirements");


struct FRequirements_HasOptionalRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		AITEST_FALSE("Empty requirements have no optional requirements", Requirements.HasOptionalRequirements());

		// Add "Optional" fragment
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Requirements with Optional fragment have optional requirements", Requirements.HasOptionalRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_HasOptionalRequirements, "System.Mass.Stress.Requirements.State.HasOptionalRequirements");


struct FRequirements_HasSparseRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		AITEST_FALSE("Empty requirements have no sparse requirements", Requirements.HasSparseRequirements());

		// Use unified API - AddRequirement should detect sparse element and route appropriately
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Requirements with sparse element have sparse requirements", Requirements.HasSparseRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_HasSparseRequirements, "System.Mass.Stress.Requirements.State.HasSparseRequirements");


struct FRequirements_Reset : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Add various requirements - use unified API for sparse element
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_FALSE("Requirements should not be empty before reset", Requirements.IsEmpty());

		Requirements.Reset();
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Requirements should be empty after reset", Requirements.IsEmpty());
		AITEST_FALSE("Should not have positive requirements after reset", Requirements.HasPositiveRequirements());
		AITEST_FALSE("Should not have sparse requirements after reset", Requirements.HasSparseRequirements());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_Reset, "System.Mass.Stress.Requirements.State.Reset");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Archetype Matching Tests
//----------------------------------------------------------------------//

struct FRequirements_DoesArchetypeMatchRequirements_AllFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		// IntsArchetype has FTestFragment_Int but not FTestFragment_Float
		AITEST_FALSE("IntsArchetype should not match (missing Float)",
			Requirements.DoesArchetypeMatchRequirements(IntsArchetype));

		// FloatsIntsArchetype has both
		AITEST_TRUE("FloatsIntsArchetype should match",
			Requirements.DoesArchetypeMatchRequirements(FloatsIntsArchetype));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_DoesArchetypeMatchRequirements_AllFragments, "System.Mass.Stress.Requirements.Match.AllFragments");


struct FRequirements_DoesArchetypeMatchRequirements_AnyFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Any);
		Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Any);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		// IntsArchetype has Int
		AITEST_TRUE("IntsArchetype should match (has Int)",
			Requirements.DoesArchetypeMatchRequirements(IntsArchetype));

		// FloatsArchetype has Float
		AITEST_TRUE("FloatsArchetype should match (has Float)",
			Requirements.DoesArchetypeMatchRequirements(FloatsArchetype));

		// EmptyArchetype has neither
		AITEST_FALSE("EmptyArchetype should not match (has neither)",
			Requirements.DoesArchetypeMatchRequirements(EmptyArchetype));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_DoesArchetypeMatchRequirements_AnyFragments, "System.Mass.Stress.Requirements.Match.AnyFragments");


struct FRequirements_DoesArchetypeMatchRequirements_NoneFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::None, EMassFragmentPresence::None);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		// IntsArchetype has Int but not Float - should match
		AITEST_TRUE("IntsArchetype should match (has Int, no Float)",
			Requirements.DoesArchetypeMatchRequirements(IntsArchetype));

		// FloatsIntsArchetype has both Int and Float - should not match (has Float which is excluded)
		AITEST_FALSE("FloatsIntsArchetype should not match (has excluded Float)",
			Requirements.DoesArchetypeMatchRequirements(FloatsIntsArchetype));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_DoesArchetypeMatchRequirements_NoneFragments, "System.Mass.Stress.Requirements.Match.NoneFragments");


struct FRequirements_DoesArchetypeMatchRequirements_Tags : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		// Create archetype with tag
		FMassArchetypeCompositionDescriptor CompositionWithTag;
		CompositionWithTag.Add<FTestFragment_Int>();
		CompositionWithTag.Add<FTestTag_A>();

		FMassArchetypeHandle ArchetypeWithTag = EntityManager->CreateArchetype(CompositionWithTag, {});

		FMassFragmentRequirements Requirements(EntityManager);
		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Archetype with tag should match",
			Requirements.DoesArchetypeMatchRequirements(ArchetypeWithTag));

		AITEST_FALSE("IntsArchetype (no tag) should not match",
			Requirements.DoesArchetypeMatchRequirements(IntsArchetype));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_DoesArchetypeMatchRequirements_Tags, "System.Mass.Stress.Requirements.Match.Tags");

struct FRequirements_DoesArchetypeMatchRequirements_AnyFragmentsAndTags : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		// Query has FTestFragment_Int and FTestTag_A both as Any.
		// An archetype should match if it has either one, not necessarily both.
		FMassFragmentRequirements Requirements(EntityManager);
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Any);
		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::Any);
		Requirements.CheckValidity();

		FMassArchetypeCompositionDescriptor FragmentOnlyComposition;
		FragmentOnlyComposition.Add<FTestFragment_Int>();
		FMassArchetypeHandle FragmentOnlyArchetype = EntityManager->CreateArchetype(FragmentOnlyComposition);

		FMassArchetypeCompositionDescriptor TagOnlyComposition;
		TagOnlyComposition.Add<FTestTag_A>();
		FMassArchetypeHandle TagOnlyArchetype = EntityManager->CreateArchetype(TagOnlyComposition);

		AITEST_TRUE("Archetype with matching fragment but no matching tags should match",
			Requirements.DoesArchetypeMatchRequirements(FragmentOnlyArchetype));

		AITEST_TRUE("Archetype with matching tag but no matching fragments should match",
			Requirements.DoesArchetypeMatchRequirements(TagOnlyArchetype));

		AITEST_FALSE("Archetype with neither matching fragment nor tag should not match",
			Requirements.DoesArchetypeMatchRequirements(FloatsArchetype));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_DoesArchetypeMatchRequirements_AnyFragmentsAndTags, "System.Mass.Stress.Requirements.Match.AnyFragmentsAndTags");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - AddElementRequirement Tests
//----------------------------------------------------------------------//

struct FRequirements_AddElementRequirement_Fragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// AddElementRequirement should route fragments to AddRequirement
		Requirements.AddElementRequirement(FTestFragment_Int::StaticStruct(), EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Fragment should be added to RequiredAllFragments",
			Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddElementRequirement_Fragment, "System.Mass.Stress.Requirements.Element.AddElementRequirement.Fragment");


struct FRequirements_AddElementRequirement_Tag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// AddElementRequirement should route tags to AddTagRequirement
		Requirements.AddElementRequirement(FTestTag_A::StaticStruct(), EMassFragmentAccess::None, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Tag should be added to RequiredAllTags",
			Requirements.GetRequiredAllTags().Contains<FTestTag_A>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddElementRequirement_Tag, "System.Mass.Stress.Requirements.Element.AddElementRequirement.Tag");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Chunk Fragment Requirements Tests
//----------------------------------------------------------------------//

struct FRequirements_AddChunkRequirement_All : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredAllChunkFragments should contain the chunk fragment",
			Requirements.GetRequiredAllChunkFragments().Contains<FTestChunkFragment_Int>());

		TConstArrayView<FMassFragmentRequirementDescription> ChunkRequirements = Requirements.GetChunkFragmentRequirements();
		AITEST_EQUAL("Should have one chunk fragment requirement", ChunkRequirements.Num(), 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddChunkRequirement_All, "System.Mass.Stress.Requirements.ChunkFragment.AddChunkRequirement.All");


struct FRequirements_AddChunkRequirement_Optional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddChunkRequirement<FTestChunkFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredOptionalChunkFragments should contain the chunk fragment",
			Requirements.GetRequiredOptionalChunkFragments().Contains<FTestChunkFragment_Float>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddChunkRequirement_Optional, "System.Mass.Stress.Requirements.ChunkFragment.AddChunkRequirement.Optional");


struct FRequirements_AddChunkRequirement_None : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredNoneChunkFragments should contain the chunk fragment",
			Requirements.GetRequiredNoneChunkFragments().Contains<FTestChunkFragment_Int>());

		// None presence should not add to ChunkFragmentRequirements array
		AITEST_EQUAL("ChunkFragmentRequirements should be empty for None presence",
			Requirements.GetChunkFragmentRequirements().Num(), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddChunkRequirement_None, "System.Mass.Stress.Requirements.ChunkFragment.AddChunkRequirement.None");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Shared Fragment Requirements Tests
//----------------------------------------------------------------------//

struct FRequirements_AddSharedRequirement_All : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredAllSharedFragments should contain the shared fragment",
			Requirements.GetRequiredAllSharedFragments().Contains<FTestSharedFragment_Int>());

		TConstArrayView<FMassFragmentRequirementDescription> SharedRequirements = Requirements.GetSharedFragmentRequirements();
		AITEST_EQUAL("Should have one shared fragment requirement", SharedRequirements.Num(), 1);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddSharedRequirement_All, "System.Mass.Stress.Requirements.SharedFragment.AddSharedRequirement.All");


struct FRequirements_AddSharedRequirement_Optional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddSharedRequirement<FTestSharedFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredOptionalSharedFragments should contain the shared fragment",
			Requirements.GetRequiredOptionalSharedFragments().Contains<FTestSharedFragment_Float>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddSharedRequirement_Optional, "System.Mass.Stress.Requirements.SharedFragment.AddSharedRequirement.Optional");


struct FRequirements_AddSharedRequirement_None : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::None, EMassFragmentPresence::None);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredNoneSharedFragments should contain the shared fragment",
			Requirements.GetRequiredNoneSharedFragments().Contains<FTestSharedFragment_Int>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddSharedRequirement_None, "System.Mass.Stress.Requirements.SharedFragment.AddSharedRequirement.None");


struct FRequirements_AddConstSharedRequirement_All : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredAllConstSharedFragments should contain the const shared fragment",
			Requirements.GetRequiredAllConstSharedFragments().Contains<FTestConstSharedFragment_Int>());

		TConstArrayView<FMassFragmentRequirementDescription> ConstSharedRequirements = Requirements.GetConstSharedFragmentRequirements();
		AITEST_EQUAL("Should have one const shared fragment requirement", ConstSharedRequirements.Num(), 1);
		// Const shared fragments are always ReadOnly
		AITEST_EQUAL("Access mode should be ReadOnly", ConstSharedRequirements[0].AccessMode, EMassFragmentAccess::ReadOnly);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddConstSharedRequirement_All, "System.Mass.Stress.Requirements.ConstSharedFragment.AddConstSharedRequirement.All");


struct FRequirements_AddConstSharedRequirement_Optional : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Float>(EMassFragmentPresence::Optional);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredOptionalConstSharedFragments should contain the const shared fragment",
			Requirements.GetRequiredOptionalConstSharedFragments().Contains<FTestConstSharedFragment_Float>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddConstSharedRequirement_Optional, "System.Mass.Stress.Requirements.ConstSharedFragment.AddConstSharedRequirement.Optional");


struct FRequirements_AddConstSharedRequirement_None : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::None);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("RequiredNoneConstSharedFragments should contain the const shared fragment",
			Requirements.GetRequiredNoneConstSharedFragments().Contains<FTestConstSharedFragment_Int>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_AddConstSharedRequirement_None, "System.Mass.Stress.Requirements.ConstSharedFragment.AddConstSharedRequirement.None");


//----------------------------------------------------------------------//
// FMassFragmentRequirements - Comprehensive Mixed Tests
//----------------------------------------------------------------------//

struct FRequirements_MixedRequirements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Add various requirement types - use unified API for sparse element
		Requirements.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
		Requirements.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All);
		Requirements.AddTagRequirement<FTestTag_B>(EMassFragmentPresence::None);
		Requirements.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddChunkRequirement<FTestChunkFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddSharedRequirement<FTestSharedFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.AddConstSharedRequirement<FTestConstSharedFragment_Int>(EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		// Verify all requirements are tracked
		AITEST_FALSE("Requirements should not be empty", Requirements.IsEmpty());
		AITEST_TRUE("Should have positive requirements", Requirements.HasPositiveRequirements());
		AITEST_TRUE("Should have negative requirements (None tag)", Requirements.HasNegativeRequirements());
		AITEST_TRUE("Should have optional requirements", Requirements.HasOptionalRequirements());
		AITEST_TRUE("Should have sparse requirements", Requirements.HasSparseRequirements());

		// Verify specific requirements
		AITEST_TRUE("Fragment Int in All", Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_TRUE("Fragment Float in Optional", Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
		AITEST_TRUE("Tag A in All", Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_TRUE("Tag B in None", Requirements.GetRequiredNoneTags().Contains<FTestTag_B>());
		AITEST_TRUE("Sparse Int in All", Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Chunk Int in All", Requirements.GetRequiredAllChunkFragments().Contains<FTestChunkFragment_Int>());
		AITEST_TRUE("Shared Int in All", Requirements.GetRequiredAllSharedFragments().Contains<FTestSharedFragment_Int>());
		AITEST_TRUE("ConstShared Int in All", Requirements.GetRequiredAllConstSharedFragments().Contains<FTestConstSharedFragment_Int>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_MixedRequirements, "System.Mass.Stress.Requirements.Mixed.Comprehensive");


struct FRequirements_Chaining : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		FMassFragmentRequirements Requirements(EntityManager);

		// Test method chaining - use unified API for sparse element
		Requirements
			.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All)
			.AddRequirement<FTestFragment_Float>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional)
			.AddTagRequirement<FTestTag_A>(EMassFragmentPresence::All)
			.AddRequirement<FTestFragment_SparseInt>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::All);
		Requirements.CheckValidity(); // force re-caching of "HasXRequirements" cached properties

		AITEST_TRUE("Fragment Int added via chaining", Requirements.GetRequiredAllFragments().Contains<FTestFragment_Int>());
		AITEST_TRUE("Fragment Float added via chaining", Requirements.GetRequiredOptionalFragments().Contains<FTestFragment_Float>());
		AITEST_TRUE("Tag A added via chaining", Requirements.GetRequiredAllTags().Contains<FTestTag_A>());
		AITEST_TRUE("Sparse Int added via chaining", Requirements.GetRequiredAllSparseElements().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FRequirements_Chaining, "System.Mass.Stress.Requirements.Mixed.Chaining");

#endif // WITH_MASSENTITY_DEBUG

} // namespace UE::Mass::Test::Requirements

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
