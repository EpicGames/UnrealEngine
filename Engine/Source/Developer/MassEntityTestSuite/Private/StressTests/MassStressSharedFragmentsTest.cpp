// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassEntityManager.h"
#include "MassEntityTypes.h"
#include "MassEntityTestTypes.h"
#include "MassArchetypeData.h"
#include "MassArchetypeTypes.h"
#include "MassEntityView.h"
#include "Algo/RandomShuffle.h"
#include "MassObserverNotificationTypes.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FMassEntityTest
{

//----------------------------------------------------------------------//
// FMassArchetypeData::SetSharedFragmentsData Tests
//----------------------------------------------------------------------//

struct FSetSharedFragmentsData_Basic : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialValue = 100;
		constexpr int32 NewValue = 200;

		// Create an archetype with shared fragment
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Verify initial shared fragment value
		const FTestSharedFragment_Int* InitialFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should have shared fragment after creation", InitialFragment);
		AITEST_EQUAL("Initial shared fragment value should match", InitialFragment->Value, InitialValue);

		// Get archetype data to call SetSharedFragmentsData
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Prepare override values
		TArray<FSharedStruct> Overrides;
		Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewValue));

		// Call SetSharedFragmentsData
		//ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, MakeArrayView(Overrides));
		ArchetypeData.SetSharedFragmentsData(Entity, MakeArrayView(Overrides));

		// Verify the entity is still valid and in the same archetype
		AITEST_TRUE("Entity should still be valid after SetSharedFragmentsData", EntityManager->IsEntityValid(Entity));
		AITEST_EQUAL("Entity should remain in the same archetype", EntityManager->GetArchetypeForEntity(Entity), Archetype);

		// Verify the shared fragment value changed
		const FTestSharedFragment_Int* UpdatedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should still have shared fragment after SetSharedFragmentsData", UpdatedFragment);
		AITEST_EQUAL("Shared fragment value should be updated", UpdatedFragment->Value, NewValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_Basic, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.Basic");


struct FSetSharedFragmentsData_MultipleEntitiesDifferentChunks : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ValueA = 10;
		constexpr int32 ValueB = 20;
		constexpr int32 ValueC = 30;

		// Create multiple entities with different initial shared fragment values
		FMassArchetypeSharedFragmentValues SharedValuesA;
		SharedValuesA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueA));
		SharedValuesA.Sort();

		FMassArchetypeSharedFragmentValues SharedValuesB;
		SharedValuesB.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueB));
		SharedValuesB.Sort();

		FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
		FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
		AITEST_EQUAL("Both entities should be in same archetype", EntityManager->GetArchetypeForEntity(EntityB), Archetype);

		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Initially there should be 2 chunks (one for each shared value)
		AITEST_EQUAL("Should have 2 chunks initially (one per shared value)", ArchetypeData.GetChunkCount(), 2);

		// Move EntityA to a new shared value (ValueC)
		TArray<FSharedStruct> OverridesForA;
		OverridesForA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueC));
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(EntityA, OverridesForA);

		// Verify EntityA moved to new chunk with ValueC
		const FTestSharedFragment_Int* FragmentA = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityA);
		AITEST_NOT_NULL("EntityA should have shared fragment", FragmentA);
		AITEST_EQUAL("EntityA's shared fragment should have new value", FragmentA->Value, ValueC);

		// Verify EntityB still has original value
		const FTestSharedFragment_Int* FragmentB = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityB);
		AITEST_NOT_NULL("EntityB should have shared fragment", FragmentB);
		AITEST_EQUAL("EntityB's shared fragment should be unchanged", FragmentB->Value, ValueB);

		// Should now have 3 chunks (ValueA empty but still exists, ValueB, ValueC)
		// Note: Empty chunks may or may not be cleaned up depending on implementation
		AITEST_TRUE("Chunk count should be at least 2", ArchetypeData.GetChunkCount() >= 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_MultipleEntitiesDifferentChunks, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.MultipleEntitiesDifferentChunks");


struct FSetSharedFragmentsData_PreservesFragmentData : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialSharedValue = 100;
		constexpr int32 NewSharedValue = 200;
		constexpr float TestFloatValue = 42.5f;

		// Create entity with both regular fragment data and shared fragment
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialSharedValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Set a specific value in the regular fragment
		FTestFragment_Float* FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should have float fragment", FloatFragment);
		FloatFragment->Value = TestFloatValue;

		// Get archetype and call SetSharedFragmentsData
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FSharedStruct> Overrides;
		Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedValue));
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

		// Verify regular fragment data is preserved
		FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should still have float fragment after move", FloatFragment);
		AITEST_EQUAL("Float fragment value should be preserved after SetSharedFragmentsData", FloatFragment->Value, TestFloatValue);

		// Verify shared fragment was updated
		const FTestSharedFragment_Int* SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should have shared fragment", SharedFragment);
		AITEST_EQUAL("Shared fragment should have new value", SharedFragment->Value, NewSharedValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_PreservesFragmentData, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.PreservesFragmentData");


struct FSetSharedFragmentsData_MultipleSharedFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialIntValue = 100;
		constexpr float InitialFloatValue = 1.5f;
		constexpr int32 NewIntValue = 200;
		constexpr float NewFloatValue = 2.5f;

		// Create entity with multiple shared fragments
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialIntValue));
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Float>(InitialFloatValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Verify initial values
		const FTestSharedFragment_Int* IntFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		const FTestSharedFragment_Float* FloatFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should have int shared fragment", IntFragment);
		AITEST_NOT_NULL("Entity should have float shared fragment", FloatFragment);
		AITEST_EQUAL("Initial int shared fragment value", IntFragment->Value, InitialIntValue);
		AITEST_EQUAL("Initial float shared fragment value", FloatFragment->Value, InitialFloatValue);

		// Get archetype and update both shared fragments
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FSharedStruct> Overrides;
		Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewIntValue));
		Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Float>(NewFloatValue));
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

		// Verify both shared fragments updated
		IntFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		FloatFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should still have int shared fragment", IntFragment);
		AITEST_NOT_NULL("Entity should still have float shared fragment", FloatFragment);
		AITEST_EQUAL("Int shared fragment should have new value", IntFragment->Value, NewIntValue);
		AITEST_EQUAL("Float shared fragment should have new value", FloatFragment->Value, NewFloatValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_MultipleSharedFragments, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.MultipleSharedFragments");


struct FSetSharedFragmentsData_PartialUpdate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialIntValue = 100;
		constexpr float InitialFloatValue = 1.5f;
		constexpr int32 NewIntValue = 200;

		// Create entity with multiple shared fragments
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialIntValue));
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Float>(InitialFloatValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Get archetype and update only the int shared fragment
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FSharedStruct> Overrides;
		Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewIntValue));
		// Note: Not adding FTestSharedFragment_Float to overrides
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

		// Verify int shared fragment updated
		const FTestSharedFragment_Int* IntFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should have int shared fragment", IntFragment);
		AITEST_EQUAL("Int shared fragment should have new value", IntFragment->Value, NewIntValue);

		// Verify float shared fragment kept its original value
		const FTestSharedFragment_Float* FloatFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should have float shared fragment", FloatFragment);
		AITEST_EQUAL("Float shared fragment should keep original value", FloatFragment->Value, InitialFloatValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_PartialUpdate, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.PartialUpdate");


struct FSetSharedFragmentsData_MoveToExistingChunk : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ValueA = 10;
		constexpr int32 ValueB = 20;

		// Create two entities with different shared values - this creates two chunks
		FMassArchetypeSharedFragmentValues SharedValuesA;
		SharedValuesA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueA));
		SharedValuesA.Sort();

		FMassArchetypeSharedFragmentValues SharedValuesB;
		FSharedStruct SharedValueB = FSharedStruct::Make<FTestSharedFragment_Int>(ValueB);
		SharedValuesB.Add(SharedValueB);
		SharedValuesB.Sort();

		FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
		FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		const int32 InitialChunkCount = ArchetypeData.GetChunkCount();
		AITEST_EQUAL("Should have 2 chunks initially", InitialChunkCount, 2);

		// Move EntityA to the same shared value as EntityB
		TArray<FSharedStruct> Overrides;
		//Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueB));
		Overrides.Add(SharedValueB);
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(EntityA, Overrides);

		// Verify EntityA now has ValueB
		const FTestSharedFragment_Int* FragmentA = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityA);
		AITEST_NOT_NULL("EntityA should have shared fragment", FragmentA);
		AITEST_EQUAL("EntityA should now have ValueB", FragmentA->Value, ValueB);

		// Verify both entities have the same shared fragment instance (they're in the same chunk)
		const FTestSharedFragment_Int* FragmentB = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityB);
		AITEST_NOT_NULL("EntityB should have shared fragment", FragmentB);
		AITEST_EQUAL("EntityA and EntityB should share the same shared fragment instance", FragmentA, FragmentB);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_MoveToExistingChunk, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.MoveToExistingChunk");


struct FSetSharedFragmentsData_WithSparseElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialSharedValue = 100;
		constexpr int32 NewSharedValue = 200;
		constexpr int32 SparseIntValue = 42;

		// Create entity with shared fragment
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialSharedValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Add a sparse element to the entity
		FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
		SparseInt.Value = SparseIntValue;

		// Verify sparse element exists
		AITEST_TRUE("Entity should have sparse element before SetSharedFragmentsData",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		FMassEntityView EntityView(*EntityManager, Entity);

		const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Sparse fragment should exist", SparseFragment);
		AITEST_EQUAL("Sparse fragment initial value", SparseFragment->Value, SparseIntValue);

		// Get archetype and call SetSharedFragmentsData
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FSharedStruct> Overrides;
		Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedValue));
		// this operation invalidates the entity view
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, Overrides);

		// Verify sparse element is preserved after the move
		AITEST_TRUE("Entity should still have sparse element after SetSharedFragmentsData",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		FMassEntityView EntityView2(*EntityManager, Entity);
		SparseFragment = EntityView2.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Sparse fragment should still exist after move", SparseFragment);
		AITEST_EQUAL("Sparse fragment value should be preserved", SparseFragment->Value, SparseIntValue);

		// Verify shared fragment was updated
		const FTestSharedFragment_Int* SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Shared fragment should exist", SharedFragment);
		AITEST_EQUAL("Shared fragment should have new value", SharedFragment->Value, NewSharedValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_WithSparseElements, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.WithSparseElements");


struct FSetSharedFragmentsData_ChunkBoundary : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ValueA = 10;
		constexpr int32 ValueB = 20;

		// Create enough entities to fill more than one chunk with the same shared value
		FMassArchetypeSharedFragmentValues SharedValuesA;
		SharedValuesA.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueA));
		SharedValuesA.Sort();

		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();
		const int32 NumEntitiesToCreate = FMath::FloorToInt32(static_cast<float>(EntitiesPerChunk) * 1.5f); // Create 1.5 chunks worth

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, SharedValuesA, NumEntitiesToCreate, Entities);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entities[0]);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Move an entity from the middle of the batch to a new shared value
		const int32 EntityToMoveIndex = NumEntitiesToCreate / 2;
		FMassEntityHandle EntityToMove = Entities[EntityToMoveIndex];

		TArray<FSharedStruct> Overrides;
		Overrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(ValueB));
		ArchetypeData.SetSharedFragmentsData<FSharedStruct>(EntityToMove, Overrides);

		// Verify the moved entity has the new value
		const FTestSharedFragment_Int* MovedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(EntityToMove);
		AITEST_NOT_NULL("Moved entity should have shared fragment", MovedFragment);
		AITEST_EQUAL("Moved entity should have new shared value", MovedFragment->Value, ValueB);

		// Verify other entities still have original value
		for (int32 i = 0; i < NumEntitiesToCreate; ++i)
		{
			if (i != EntityToMoveIndex)
			{
				const FTestSharedFragment_Int* Fragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entities[i]);
				AITEST_NOT_NULL("Other entity should have shared fragment", Fragment);
				AITEST_EQUAL("Other entity should have original shared value", Fragment->Value, ValueA);
			}
		}

		// Total entity count should be preserved
		AITEST_EQUAL("Total entity count should be preserved", ArchetypeData.GetNumEntities(), NumEntitiesToCreate);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_ChunkBoundary, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.ChunkBoundary");


//----------------------------------------------------------------------//
// FMassArchetypeData::SetSharedFragmentsData Tests - Const Shared Fragments
//----------------------------------------------------------------------//

struct FSetConstSharedFragmentsData_Basic : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialValue = 100;
		constexpr int32 NewValue = 200;

		// Create an archetype with const shared fragment
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Verify initial const shared fragment value
		const FTestConstSharedFragment_Int* InitialFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should have const shared fragment after creation", InitialFragment);
		AITEST_EQUAL("Initial const shared fragment value should match", InitialFragment->Value, InitialValue);

		// Get archetype data to call SetSharedFragmentsData
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Prepare override values
		TArray<FConstSharedStruct> Overrides;
		Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewValue));

		// Call SetSharedFragmentsData with const shared fragment
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

		// Verify the entity is still valid and in the same archetype
		AITEST_TRUE("Entity should still be valid after SetSharedFragmentsData", EntityManager->IsEntityValid(Entity));
		AITEST_EQUAL("Entity should remain in the same archetype", EntityManager->GetArchetypeForEntity(Entity), Archetype);

		// Verify the const shared fragment value changed
		const FTestConstSharedFragment_Int* UpdatedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should still have const shared fragment after SetSharedFragmentsData", UpdatedFragment);
		AITEST_EQUAL("Const shared fragment value should be updated", UpdatedFragment->Value, NewValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_Basic, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.Basic");


struct FSetConstSharedFragmentsData_MultipleEntitiesDifferentChunks : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ValueA = 10;
		constexpr int32 ValueB = 20;
		constexpr int32 ValueC = 30;

		// Create multiple entities with different initial const shared fragment values
		FMassArchetypeSharedFragmentValues SharedValuesA;
		SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueA));
		SharedValuesA.Sort();

		FMassArchetypeSharedFragmentValues SharedValuesB;
		SharedValuesB.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueB));
		SharedValuesB.Sort();

		FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
		FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
		AITEST_EQUAL("Both entities should be in same archetype", EntityManager->GetArchetypeForEntity(EntityB), Archetype);

		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Initially there should be 2 chunks (one for each const shared value)
		AITEST_EQUAL("Should have 2 chunks initially (one per const shared value)", ArchetypeData.GetChunkCount(), 2);

		// Move EntityA to a new const shared value (ValueC)
		TArray<FConstSharedStruct> OverridesForA;
		OverridesForA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueC));
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(EntityA, OverridesForA);

		// Verify EntityA moved to new chunk with ValueC
		const FTestConstSharedFragment_Int* FragmentA = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityA);
		AITEST_NOT_NULL("EntityA should have const shared fragment", FragmentA);
		AITEST_EQUAL("EntityA's const shared fragment should have new value", FragmentA->Value, ValueC);

		// Verify EntityB still has original value
		const FTestConstSharedFragment_Int* FragmentB = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityB);
		AITEST_NOT_NULL("EntityB should have const shared fragment", FragmentB);
		AITEST_EQUAL("EntityB's const shared fragment should be unchanged", FragmentB->Value, ValueB);

		// Should now have at least 2 chunks
		AITEST_TRUE("Chunk count should be at least 2", ArchetypeData.GetChunkCount() >= 2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_MultipleEntitiesDifferentChunks, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.MultipleEntitiesDifferentChunks");


struct FSetConstSharedFragmentsData_PreservesFragmentData : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialSharedValue = 100;
		constexpr int32 NewSharedValue = 200;
		constexpr float TestFloatValue = 42.5f;

		// Create entity with both regular fragment data and const shared fragment
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialSharedValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Set a specific value in the regular fragment
		FTestFragment_Float* FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should have float fragment", FloatFragment);
		FloatFragment->Value = TestFloatValue;

		// Get archetype and call SetSharedFragmentsData
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FConstSharedStruct> Overrides;
		Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewSharedValue));
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

		// Verify regular fragment data is preserved
		FloatFragment = EntityManager->GetFragmentDataPtr<FTestFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should still have float fragment after move", FloatFragment);
		AITEST_EQUAL("Float fragment value should be preserved after SetSharedFragmentsData", FloatFragment->Value, TestFloatValue);

		// Verify const shared fragment was updated
		const FTestConstSharedFragment_Int* SharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should have const shared fragment", SharedFragment);
		AITEST_EQUAL("Const shared fragment should have new value", SharedFragment->Value, NewSharedValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_PreservesFragmentData, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.PreservesFragmentData");


struct FSetConstSharedFragmentsData_MultipleConstSharedFragments : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialIntValue = 100;
		constexpr float InitialFloatValue = 1.5f;
		constexpr int32 NewIntValue = 200;
		constexpr float NewFloatValue = 2.5f;

		// Create entity with multiple const shared fragments
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialIntValue));
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(InitialFloatValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Verify initial values
		const FTestConstSharedFragment_Int* IntFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		const FTestConstSharedFragment_Float* FloatFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should have int const shared fragment", IntFragment);
		AITEST_NOT_NULL("Entity should have float const shared fragment", FloatFragment);
		AITEST_EQUAL("Initial int const shared fragment value", IntFragment->Value, InitialIntValue);
		AITEST_EQUAL("Initial float const shared fragment value", FloatFragment->Value, InitialFloatValue);

		// Get archetype and update both const shared fragments
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FConstSharedStruct> Overrides;
		Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewIntValue));
		Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(NewFloatValue));
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

		// Verify both const shared fragments updated
		IntFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		FloatFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should still have int const shared fragment", IntFragment);
		AITEST_NOT_NULL("Entity should still have float const shared fragment", FloatFragment);
		AITEST_EQUAL("Int const shared fragment should have new value", IntFragment->Value, NewIntValue);
		AITEST_EQUAL("Float const shared fragment should have new value", FloatFragment->Value, NewFloatValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_MultipleConstSharedFragments, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.MultipleConstSharedFragments");


struct FSetConstSharedFragmentsData_PartialUpdate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialIntValue = 100;
		constexpr float InitialFloatValue = 1.5f;
		constexpr int32 NewIntValue = 200;

		// Create entity with multiple const shared fragments
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialIntValue));
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(InitialFloatValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Get archetype and update only the int const shared fragment
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FConstSharedStruct> Overrides;
		Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewIntValue));
		// Note: Not adding FTestConstSharedFragment_Float to overrides
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

		// Verify int const shared fragment updated
		const FTestConstSharedFragment_Int* IntFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should have int const shared fragment", IntFragment);
		AITEST_EQUAL("Int const shared fragment should have new value", IntFragment->Value, NewIntValue);

		// Verify float const shared fragment kept its original value
		const FTestConstSharedFragment_Float* FloatFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);
		AITEST_NOT_NULL("Entity should have float const shared fragment", FloatFragment);
		AITEST_EQUAL("Float const shared fragment should keep original value", FloatFragment->Value, InitialFloatValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_PartialUpdate, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.PartialUpdate");


struct FSetConstSharedFragmentsData_MoveToExistingChunk : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ValueA = 10;
		constexpr int32 ValueB = 20;

		// Create two entities with different const shared values - this creates two chunks
		FMassArchetypeSharedFragmentValues SharedValuesA;
		SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueA));
		SharedValuesA.Sort();

		FMassArchetypeSharedFragmentValues SharedValuesB;
		FConstSharedStruct SharedValueB = FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueB);
		SharedValuesB.Add(SharedValueB);
		SharedValuesB.Sort();

		FMassEntityHandle EntityA = EntityManager->CreateEntity(FloatsArchetype, SharedValuesA);
		FMassEntityHandle EntityB = EntityManager->CreateEntity(FloatsArchetype, SharedValuesB);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(EntityA);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		const int32 InitialChunkCount = ArchetypeData.GetChunkCount();
		AITEST_EQUAL("Should have 2 chunks initially", InitialChunkCount, 2);

		// Move EntityA to the same const shared value as EntityB
		TArray<FConstSharedStruct> Overrides;
		Overrides.Add(SharedValueB);
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(EntityA, Overrides);

		// Verify EntityA now has ValueB
		const FTestConstSharedFragment_Int* FragmentA = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityA);
		AITEST_NOT_NULL("EntityA should have const shared fragment", FragmentA);
		AITEST_EQUAL("EntityA should now have ValueB", FragmentA->Value, ValueB);

		// Verify both entities have the same const shared fragment instance (they're in the same chunk)
		const FTestConstSharedFragment_Int* FragmentB = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityB);
		AITEST_NOT_NULL("EntityB should have const shared fragment", FragmentB);
		AITEST_EQUAL("EntityA and EntityB should share the same const shared fragment instance", FragmentA, FragmentB);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_MoveToExistingChunk, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.MoveToExistingChunk");


struct FSetConstSharedFragmentsData_WithSparseElements : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialSharedValue = 100;
		constexpr int32 NewSharedValue = 200;
		constexpr int32 SparseIntValue = 42;

		// Create entity with const shared fragment
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialSharedValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Add a sparse element to the entity
		FTestFragment_SparseInt& SparseInt = EntityManager->AddSparseElementToEntity<FTestFragment_SparseInt>(Entity);
		SparseInt.Value = SparseIntValue;

		// Verify sparse element exists
		AITEST_TRUE("Entity should have sparse element before SetSharedFragmentsData",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		FMassEntityView EntityView(*EntityManager, Entity);

		const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Sparse fragment should exist", SparseFragment);
		AITEST_EQUAL("Sparse fragment initial value", SparseFragment->Value, SparseIntValue);

		// Get archetype and call SetSharedFragmentsData
		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		TArray<FConstSharedStruct> Overrides;
		Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewSharedValue));
		// this operation invalidates the entity view
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, Overrides);

		// Verify sparse element is preserved after the move
		AITEST_TRUE("Entity should still have sparse element after SetSharedFragmentsData",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		FMassEntityView NewEntityView(*EntityManager, Entity);
		SparseFragment = NewEntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Sparse fragment should still exist after move", SparseFragment);
		AITEST_EQUAL("Sparse fragment value should be preserved", SparseFragment->Value, SparseIntValue);

		// Verify const shared fragment was updated
		const FTestConstSharedFragment_Int* SharedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Const shared fragment should exist", SharedFragment);
		AITEST_EQUAL("Const shared fragment should have new value", SharedFragment->Value, NewSharedValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_WithSparseElements, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.WithSparseElements");


struct FSetConstSharedFragmentsData_ChunkBoundary : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 ValueA = 10;
		constexpr int32 ValueB = 20;

		// Create enough entities to fill more than one chunk with the same const shared value
		FMassArchetypeSharedFragmentValues SharedValuesA;
		SharedValuesA.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueA));
		SharedValuesA.Sort();

		const int32 EntitiesPerChunk = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(FloatsArchetype).GetNumEntitiesPerChunk();
		const int32 NumEntitiesToCreate = FMath::FloorToInt32(static_cast<float>(EntitiesPerChunk) * 1.5f); // Create 1.5 chunks worth

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(FloatsArchetype, SharedValuesA, NumEntitiesToCreate, Entities);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entities[0]);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Move an entity from the middle of the batch to a new const shared value
		const int32 EntityToMoveIndex = NumEntitiesToCreate / 2;
		FMassEntityHandle EntityToMove = Entities[EntityToMoveIndex];

		TArray<FConstSharedStruct> Overrides;
		Overrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ValueB));
		ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(EntityToMove, Overrides);

		// Verify the moved entity has the new value
		const FTestConstSharedFragment_Int* MovedFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(EntityToMove);
		AITEST_NOT_NULL("Moved entity should have const shared fragment", MovedFragment);
		AITEST_EQUAL("Moved entity should have new const shared value", MovedFragment->Value, ValueB);

		// Verify other entities still have original value
		for (int32 i = 0; i < NumEntitiesToCreate; ++i)
		{
			if (i != EntityToMoveIndex)
			{
				const FTestConstSharedFragment_Int* Fragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entities[i]);
				AITEST_NOT_NULL("Other entity should have const shared fragment", Fragment);
				AITEST_EQUAL("Other entity should have original const shared value", Fragment->Value, ValueA);
			}
		}

		// Total entity count should be preserved
		AITEST_EQUAL("Total entity count should be preserved", ArchetypeData.GetNumEntities(), NumEntitiesToCreate);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetConstSharedFragmentsData_ChunkBoundary, "System.Mass.Stress.SharedFragments.SetConstSharedFragmentsData.ChunkBoundary");


//----------------------------------------------------------------------//
// FMassArchetypeData::SetSharedFragmentsData Tests - Mixed Shared/Const
//----------------------------------------------------------------------//

struct FSetSharedFragmentsData_MixedSharedAndConst : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialSharedIntValue = 100;
		constexpr int32 InitialConstIntValue = 200;
		constexpr int32 NewSharedIntValue = 150;
		constexpr int32 NewConstIntValue = 250;

		// Create entity with both shared and const shared fragments
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(InitialSharedIntValue));
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(InitialConstIntValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		// Verify initial values
		const FTestSharedFragment_Int* SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		const FTestConstSharedFragment_Int* ConstFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should have shared fragment", SharedFragment);
		AITEST_NOT_NULL("Entity should have const shared fragment", ConstFragment);
		AITEST_EQUAL("Initial shared fragment value", SharedFragment->Value, InitialSharedIntValue);
		AITEST_EQUAL("Initial const shared fragment value", ConstFragment->Value, InitialConstIntValue);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Update only the mutable shared fragment
		{
			TArray<FSharedStruct> SharedOverrides;
			SharedOverrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedIntValue));
			ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, SharedOverrides);
		}

		// Verify shared fragment updated, const unchanged
		SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		ConstFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should still have shared fragment", SharedFragment);
		AITEST_NOT_NULL("Entity should still have const shared fragment", ConstFragment);
		AITEST_EQUAL("Shared fragment should have new value", SharedFragment->Value, NewSharedIntValue);
		AITEST_EQUAL("Const shared fragment should be unchanged", ConstFragment->Value, InitialConstIntValue);

		// Now update only the const shared fragment
		{
			TArray<FConstSharedStruct> ConstOverrides;
			ConstOverrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewConstIntValue));
			ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, ConstOverrides);
		}

		// Verify both are now updated
		SharedFragment = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		ConstFragment = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		AITEST_NOT_NULL("Entity should still have shared fragment", SharedFragment);
		AITEST_NOT_NULL("Entity should still have const shared fragment", ConstFragment);
		AITEST_EQUAL("Shared fragment should retain its value", SharedFragment->Value, NewSharedIntValue);
		AITEST_EQUAL("Const shared fragment should have new value", ConstFragment->Value, NewConstIntValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_MixedSharedAndConst, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.MixedSharedAndConst");


struct FSetSharedFragmentsData_MixedPreservesOtherType : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 SharedIntValue = 100;
		constexpr float SharedFloatValue = 1.5f;
		constexpr int32 ConstIntValue = 200;
		constexpr float ConstFloatValue = 2.5f;
		constexpr int32 NewSharedIntValue = 150;
		constexpr int32 NewConstIntValue = 250;

		// Create entity with multiple shared and const shared fragments
		FMassArchetypeSharedFragmentValues SharedValues;
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Int>(SharedIntValue));
		SharedValues.Add(FSharedStruct::Make<FTestSharedFragment_Float>(SharedFloatValue));
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(ConstIntValue));
		SharedValues.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Float>(ConstFloatValue));
		SharedValues.Sort();

		FMassEntityHandle Entity = EntityManager->CreateEntity(FloatsArchetype, SharedValues);

		FMassArchetypeHandle Archetype = EntityManager->GetArchetypeForEntity(Entity);
		FMassArchetypeData& ArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(Archetype);

		// Update only one mutable shared fragment (Int), keeping Float unchanged
		{
			TArray<FSharedStruct> SharedOverrides;
			SharedOverrides.Add(FSharedStruct::Make<FTestSharedFragment_Int>(NewSharedIntValue));
			ArchetypeData.SetSharedFragmentsData<FSharedStruct>(Entity, SharedOverrides);
		}

		// Verify only the Int shared fragment changed
		const FTestSharedFragment_Int* SharedInt = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		const FTestSharedFragment_Float* SharedFloat = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
		const FTestConstSharedFragment_Int* ConstInt = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		const FTestConstSharedFragment_Float* ConstFloat = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);

		AITEST_NOT_NULL("SharedInt should exist", SharedInt);
		AITEST_NOT_NULL("SharedFloat should exist", SharedFloat);
		AITEST_NOT_NULL("ConstInt should exist", ConstInt);
		AITEST_NOT_NULL("ConstFloat should exist", ConstFloat);

		AITEST_EQUAL("SharedInt should have new value", SharedInt->Value, NewSharedIntValue);
		AITEST_EQUAL("SharedFloat should be unchanged", SharedFloat->Value, SharedFloatValue);
		AITEST_EQUAL("ConstInt should be unchanged", ConstInt->Value, ConstIntValue);
		AITEST_EQUAL("ConstFloat should be unchanged", ConstFloat->Value, ConstFloatValue);

		// Update only one const shared fragment (Int), keeping Float unchanged
		{
			TArray<FConstSharedStruct> ConstOverrides;
			ConstOverrides.Add(FConstSharedStruct::Make<FTestConstSharedFragment_Int>(NewConstIntValue));
			ArchetypeData.SetSharedFragmentsData<FConstSharedStruct>(Entity, ConstOverrides);
		}

		// Re-fetch and verify
		SharedInt = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Int>(Entity);
		SharedFloat = EntityManager->GetSharedFragmentDataPtr<FTestSharedFragment_Float>(Entity);
		ConstInt = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Int>(Entity);
		ConstFloat = EntityManager->GetConstSharedFragmentDataPtr<FTestConstSharedFragment_Float>(Entity);

		AITEST_NOT_NULL("SharedInt should still exist", SharedInt);
		AITEST_NOT_NULL("SharedFloat should still exist", SharedFloat);
		AITEST_NOT_NULL("ConstInt should still exist", ConstInt);
		AITEST_NOT_NULL("ConstFloat should still exist", ConstFloat);

		AITEST_EQUAL("SharedInt should retain value", SharedInt->Value, NewSharedIntValue);
		AITEST_EQUAL("SharedFloat should still be unchanged", SharedFloat->Value, SharedFloatValue);
		AITEST_EQUAL("ConstInt should have new value", ConstInt->Value, NewConstIntValue);
		AITEST_EQUAL("ConstFloat should still be unchanged", ConstFloat->Value, ConstFloatValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSetSharedFragmentsData_MixedPreservesOtherType, "System.Mass.Stress.SharedFragments.SetSharedFragmentsData.MixedPreservesOtherType");

} // FMassEntityTest

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
