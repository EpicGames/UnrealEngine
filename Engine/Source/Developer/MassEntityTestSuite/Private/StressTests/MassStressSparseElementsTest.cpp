// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "MassArchetypeData.h"
#include "MassEntityBuilder.h"
#include "MassEntityManager.h"
#include "Mass/EntityMacros.h"
#include "MassEntityTestTypes.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassExecutor.h"

#define LOCTEXT_NAMESPACE "MassTest"

UE_DISABLE_OPTIMIZATION_SHIP

// to be enabled once this functionality gets implemented
#define WITH_ENTITY_MOVING_AFFECTING_COMPOSITION 0

namespace UE::Mass::Test::SparseElements
{
#if WITH_MASSENTITY_DEBUG

	struct FSparseElements_Storage_ConfigureType : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			UE::Mass::FSparseElementsStorage Storage;

			// Configure custom chunk size
			UE::Mass::FSparseElementsStorage::FTypeConfig Config(FTestFragment_SparseInt::StaticStruct());
			// change the default chunk size, for testing purposes.
			Config.SetElementsPerChunk(64); 

			Storage.ConfigureType(FTestFragment_SparseInt::StaticStruct(), Config);

			// Create entities beyond one chunk
			constexpr int32 NumEntities = 200;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Add sparse elements - should use configured chunk size
			for (const FMassEntityHandle& Entity : Entities)
			{
				FStructView View = Storage.AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_TRUE(TEXT("Added element with custom chunk size"), View.IsValid());
			}

			// Verify we can access all of them
			for (const FMassEntityHandle& Entity : Entities)
			{
				FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_TRUE(TEXT("Element accessible with custom chunk size"), View.IsValid());
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_Storage_ConfigureType, "System.Mass.Stress.Sparse.Storage.ConfigureType");

	struct FSparseElements_Storage_OnEntitiesDestroyed : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();

			constexpr int32 NumEntities = 10;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Add sparse elements to all
			for (const FMassEntityHandle& Entity : Entities)
			{
				FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_TRUE(TEXT("Element added successfully"), View.IsValid());
			}

			// Destroy subset - test the bug fix (return -> continue)
			TArray<FMassEntityHandle> EntitiesToDestroy = {Entities[2], Entities[5], Entities[7]};
			EntityManager->BatchDestroyEntities(EntitiesToDestroy);

			// Verify destroyed entities no longer have elements
			for (const FMassEntityHandle& Entity : EntitiesToDestroy)
			{
				FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_FALSE(TEXT("Destroyed entity has no sparse element"), View.IsValid());
			}

			// Verify remaining entities still have elements (validates bug fix)
			for (int32 Index = 0; Index < NumEntities; ++Index)
			{
				if (!EntitiesToDestroy.Contains(Entities[Index]))
				{
					FStructView View = Storage.GetMutableElementDataForEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
					AITEST_TRUE(TEXT("Remaining entity still has sparse element"), View.IsValid());
				}
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_Storage_OnEntitiesDestroyed, "System.Mass.Stress.Sparse.Storage.OnEntitiesDestroyed");

	struct FSparseElements_MultipleTypesPerEntity : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

			// Add 3 different sparse types
			FStructView IntView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
			FStructView FloatView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
			FStructView TagView = EntityManager->AddSparseElementToEntity(Entity, FTestTag_SparseA::StaticStruct());

			AITEST_TRUE(TEXT("Sparse int added"), IntView.IsValid());
			AITEST_TRUE(TEXT("Sparse float added"), FloatView.IsValid());
			AITEST_TRUE(TEXT("Added sparse tag view is invalid"), TagView.IsValid() == false);

			// Verify all are present
			FMassEntityView EntityView(*EntityManager, Entity);
			AITEST_TRUE(TEXT("Has sparse int"), EntityView.HasElement<FTestFragment_SparseInt>());
			AITEST_TRUE(TEXT("Has sparse float"), EntityView.HasElement<FTestFragment_SparseFloat>());
			AITEST_TRUE(TEXT("Has sparse tag"), EntityView.HasElement<FTestTag_SparseA>());

			// Remove one, verify others remain
			EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseFloat::StaticStruct());

			AITEST_TRUE(TEXT("Still has sparse int"), EntityView.HasElement<FTestFragment_SparseInt>());
			AITEST_FALSE(TEXT("No longer has sparse float"), EntityView.HasElement<FTestFragment_SparseFloat>());
			AITEST_TRUE(TEXT("Still has sparse tag"), EntityView.HasElement<FTestTag_SparseA>());

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_MultipleTypesPerEntity, "System.Mass.Stress.Sparse.MultipleTypes");

	struct FSparseElements_RemoveNonExistent : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

			// Try to remove element that was never added
			EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

			// Add, remove, try to remove again
			FStructView _ = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
			EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_RemoveNonExistent, "System.Mass.Stress.Sparse.RemoveNonExistent");

	struct FSparseElements_DataPersistence : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 20;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Add sparse fragments with unique values
			for (int32 Index = 0; Index < NumEntities; ++Index)
			{
				FStructView View = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
				FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
				Fragment->Value = Index * 100;
			}

			// Move entities to different archetype
			EntityManager->BatchChangeFragmentCompositionForEntities(
				{FMassArchetypeEntityCollection(IntsArchetype, Entities, FMassArchetypeEntityCollection::EDuplicatesHandling::NoDuplicates)},
				FMassFragmentBitSet(FTestFragment_Float::StaticStruct()),
				{}
			);

			// Verify data persisted through move
			for (int32 Index = 0; Index < NumEntities; ++Index)
			{
				FMassEntityView EntityView(*EntityManager, Entities[Index]);
				const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				AITEST_NOT_NULL(TEXT("Fragment still exists after move"), Fragment);
				AITEST_EQUAL(TEXT("Fragment value preserved"), Fragment->Value, Index * 100);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_DataPersistence, "System.Mass.Stress.Sparse.DataPersistence");

	struct FSparseElements_CrossChunkBatchOperations : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
			constexpr int32 NumChunks = 5;
			const int32 NumEntities = NumChunks * EntitiesPerChunk;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Select entities scattered across all chunks
			TArray<FMassEntityHandle> SelectedEntities;
			for (int32 ChunkIdx = 0; ChunkIdx < NumChunks; ++ChunkIdx)
			{
				// Pick 3 entities from each chunk
				for (int32 InChunkIdx = 0; InChunkIdx < 3 && (ChunkIdx * EntitiesPerChunk + InChunkIdx) < NumEntities; ++InChunkIdx)
				{
					SelectedEntities.Add(Entities[ChunkIdx * EntitiesPerChunk + InChunkIdx]);
				}
			}

			// Batch add across chunks
			FMassArchetypeEntityCollection Collection(IntsArchetype, SelectedEntities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());

			// Verify all have the element
			for (const FMassEntityHandle& Entity : SelectedEntities)
			{
				FMassEntityView EntityView(*EntityManager, Entity);
				AITEST_TRUE(TEXT("Entity has sparse element"), EntityView.HasElement<FTestFragment_SparseInt>());
			}

			// Batch remove across chunks
			EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());

			// Verify all removed
			for (const FMassEntityHandle& Entity : SelectedEntities)
			{
				FMassEntityView EntityView(*EntityManager, Entity);
				AITEST_FALSE(TEXT("Entity no longer has sparse element"), EntityView.HasElement<FTestFragment_SparseInt>());
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_CrossChunkBatchOperations, "System.Mass.Stress.Sparse.CrossChunkBatch");

	struct FSparseElements_Storage_ChunkBoundaryAccess : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			UE::Mass::FSparseElementsStorage Storage;

			// Test accesses at chunk boundaries (default 128 elements per chunk)
			constexpr int32 BoundaryIndices[] = {0, 127, 128, 255, 256, 383, 384};

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, 500, Entities);

			// Add sparse elements at boundary indices
			for (int32 BoundaryIdx : BoundaryIndices)
			{
				if (BoundaryIdx < Entities.Num())
				{
					FStructView View = Storage.AddElementToEntity(Entities[BoundaryIdx], FTestFragment_SparseInt::StaticStruct());
					AITEST_TRUE(TEXT("Boundary add succeeded"), View.IsValid());

					FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
					Fragment->Value = BoundaryIdx;
				}
			}

			// Verify all boundary elements accessible
			for (int32 BoundaryIdx : BoundaryIndices)
			{
				if (BoundaryIdx < Entities.Num())
				{
					FConstStructView View = Storage.GetElementDataForEntity(Entities[BoundaryIdx], FTestFragment_SparseInt::StaticStruct());
					AITEST_TRUE(TEXT("Boundary element accessible"), View.IsValid());

					const FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
					AITEST_NOT_NULL(TEXT("Fragment pointer valid"), Fragment);
					AITEST_EQUAL(TEXT("Boundary element value correct"), Fragment->Value, BoundaryIdx);
				}
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_Storage_ChunkBoundaryAccess, "System.Mass.Stress.Sparse.Storage.ChunkBoundary");

	struct FSparseElements_BatchOperationsEmptyInput : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			TArray<FMassEntityHandle> EmptyArray;
			FMassArchetypeEntityCollection EmptyCollection;

			// Should not crash with empty input
			EntityManager->BatchAddSparseElementToEntities({EmptyCollection}, FTestFragment_SparseInt::StaticStruct());
			EntityManager->BatchRemoveSparseElementFromEntities({EmptyCollection}, FTestFragment_SparseInt::StaticStruct());

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchOperationsEmptyInput, "System.Mass.Stress.Sparse.BatchEmpty");

	struct FSparseElements_ComplexQueryCombinations : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 20;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Group 1: Has SparseInt only (entities 0-4)
			for (int32 Index = 0; Index < 5; ++Index)
			{
				FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
			}

			// Group 2: Has SparseFloat only (entities 5-9)
			for (int32 Index = 5; Index < 10; ++Index)
			{
				FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseFloat::StaticStruct());
			}

			// Group 3: Has both (entities 10-14)
			for (int32 Index = 10; Index < 15; ++Index)
			{
				FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
				FStructView __ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseFloat::StaticStruct());
			}

			// Group 4: Has neither (entities 15-19)

			// Query 1: Entities with both sparse elements
			{
				FMassEntityQuery Query(EntityManager);
				// required since Queries don't support sparse-only requirements
				Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
				Query.AddSparseRequirement<FTestFragment_SparseInt>();
				Query.AddSparseRequirement<FTestFragment_SparseFloat>();

				FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
				int32 Count = 0;
				Query.ForEachEntityChunk(ExecutionContext, [&Count](FMassExecutionContext& Context)
				{
					for (FMassExecutionContext::FSparseEntityIterator It = Context.CreateSparseEntityIterator(); It; ++It)
					{
						++Count;
					}
				});

				AITEST_EQUAL(TEXT("Query with both sparse elements"), Count, 5);
			}

			// Query 2: Entities with SparseInt but NOT SparseFloat
			{
				FMassEntityQuery Query(EntityManager);
				// required since Queries don't support sparse-only requirements
				Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
				Query.AddSparseRequirement<FTestFragment_SparseInt>();
				Query.AddSparseRequirement<FTestFragment_SparseFloat>(EMassFragmentPresence::None);

				FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
				int32 Count = 0;
				Query.ForEachEntityChunk(ExecutionContext, [&Count](FMassExecutionContext& Context)
				{
					for (FMassExecutionContext::FSparseEntityIterator It = Context.CreateSparseEntityIterator(); It; ++It)
					{
						++Count;
					}
				});

				AITEST_EQUAL(TEXT("Query with SparseInt, without SparseFloat"), Count, 5);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ComplexQueryCombinations, "System.Mass.Stress.Sparse.ComplexQuery");

	struct FSparseElements_MemoryFragmentation : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			UE::Mass::FSparseElementsStorage Storage;

			constexpr int32 NumEntities = 1000;
			constexpr int32 NumCycles = 10;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Repeatedly add and remove to create fragmentation
			for (int32 Cycle = 0; Cycle < NumCycles; ++Cycle)
			{
				// Add to all
				for (const FMassEntityHandle& Entity : Entities)
				{
					FStructView _ = Storage.AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				}

				// Remove from half (creates holes)
				for (int32 Index = 0; Index < NumEntities / 2; ++Index)
				{
					Storage.RemoveElementFromEntity<FTestFragment_SparseInt>(Entities[Index]);
				}

				// Re-add to different half
				for (int32 Index = 0; Index < NumEntities / 2; ++Index)
				{
					FStructView _ = Storage.AddElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
				}
			}

			// Final verification - all should have the element
			for (const FMassEntityHandle& Entity : Entities)
			{
				FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_TRUE(TEXT("Entity has sparse element after fragmentation"), View.IsValid());
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_MemoryFragmentation, "System.Mass.Stress.Sparse.MemoryFragmentation");
	
	struct FSparseElements_OccupationMaskFiltering : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 200;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Add to odd indices only
			for (int32 Index = 1; Index < NumEntities; Index += 2)
			{
				FStructView _ = EntityManager->AddSparseElementToEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
			}

			// Query should only find odd indices
			FMassEntityQuery Query(EntityManager);
			Query.AddRequirement<FTestFragment_Int>(EMassFragmentAccess::ReadOnly);
			Query.AddSparseRequirement<FTestFragment_SparseInt>();

			FMassExecutionContext ExecutionContext = EntityManager->CreateExecutionContext(0);
			TSet<int32> FoundIndices;

			Query.ForEachEntityChunk(ExecutionContext, [&FoundIndices](FMassExecutionContext& Context)
			{
				for (FMassExecutionContext::FSparseEntityIterator It = Context.CreateSparseEntityIterator(); It; ++It)
				{
					FoundIndices.Add(Context.GetEntities()[It].Index);
				}
			});

			// Verify iterator correctly skipped even indices
			int32 ExpectedCount = 0;
			for (int32 Index = 0; Index < NumEntities; ++Index)
			{
				const bool bShouldHave = (Index % 2 == 1);
				const bool bActuallyHas = FoundIndices.Contains(Entities[Index].Index);
				AITEST_EQUAL(TEXT("Occupation mask filtering correct"), bActuallyHas, bShouldHave);
				if (bShouldHave) ++ExpectedCount;
			}

			AITEST_EQUAL(TEXT("Found expected number of sparse elements"), FoundIndices.Num(), ExpectedCount);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_OccupationMaskFiltering, "System.Mass.Stress.Sparse.OccupationFiltering");
	
	// CRITICAL: Test that UStruct destructors are called (validates bug fix #2)
	// Note: This test requires a special fragment type that tracks construction/destruction
	// If FTestFragment_SparseInt doesn't have suitable tracking, this test may need modification
	struct FSparseElements_DestructorValidation : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			UE::Mass::FSparseElementsStorage& Storage = EntityManager->DebugGetSparseElementsStorage();

			constexpr int32 NumEntities = 10;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Add sparse fragments - they should be properly constructed
			for (const FMassEntityHandle& Entity : Entities)
			{
				FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_TRUE(TEXT("Fragment added"), View.IsValid());

				// Modify to ensure it's a real instance
				FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
				Fragment->Value = 42;
			}

			// Remove half - DestroyStruct should be called
			for (int32 Index = 0; Index < NumEntities / 2; ++Index)
			{
				EntityManager->RemoveSparseElementFromEntity<FTestFragment_SparseInt>(Entities[Index]);

				// Verify it's actually gone
				FStructView View = Storage.GetMutableElementDataForEntity(Entities[Index], FTestFragment_SparseInt::StaticStruct());
				AITEST_FALSE(TEXT("Fragment no longer exists"), View.IsValid());
			}

			// Destroy remaining entities - should call destructors via OnEntitiesDestroyed
			TArray<FMassEntityHandle> RemainingEntities;
			for (int32 Index = NumEntities / 2; Index < NumEntities; ++Index)
			{
				RemainingEntities.Add(Entities[Index]);
			}
			EntityManager->BatchDestroyEntities(RemainingEntities);

			// Verify all destroyed
			for (const FMassEntityHandle& Entity : RemainingEntities)
			{
				FStructView View = Storage.GetMutableElementDataForEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_FALSE(TEXT("Destroyed entity fragment removed"), View.IsValid());
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_DestructorValidation, "System.Mass.Stress.Sparse.DestructorValidation");
	
	struct FSparseElements_Storage_CountTracking : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			FSparseElementsStorage Storage;

			// Initially empty
			AITEST_EQUAL(TEXT("Initial count is zero"), Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), 0u);
			AITEST_FALSE(TEXT("Initially has no elements"), Storage.HasAnyElementsOfType<FTestFragment_SparseInt>());

			constexpr int32 NumEntities = 50;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Add to half
			for (int32 i = 0; i < NumEntities / 2; ++i)
			{
				FStructView _ = Storage.AddElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
			}

			AITEST_EQUAL(TEXT("Count after adding half"), Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), 25u);
			AITEST_TRUE(TEXT("Has elements after adding"), Storage.HasAnyElementsOfType<FTestFragment_SparseInt>());

			// Remove quarter
			for (int32 i = 0; i < NumEntities / 4; ++i)
			{
				Storage.RemoveElementFromEntity<FTestFragment_SparseInt>(Entities[i]);
			}

			AITEST_EQUAL(TEXT("Count after removing quarter"), Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), 13u);

			// Remove all
			for (int32 i = 0; i < NumEntities; ++i)
			{
				Storage.RemoveElementFromEntity<FTestFragment_SparseInt>(Entities[i]);
			}

			AITEST_EQUAL(TEXT("Count after removing all"), Storage.GetNumElementsOfType<FTestFragment_SparseInt>(), 0u);
			AITEST_FALSE(TEXT("No elements after removing all"), Storage.HasAnyElementsOfType<FTestFragment_SparseInt>());

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_Storage_CountTracking, "System.Mass.Stress.Sparse.InstanceCountTracking");

	struct FSparseElements_AddRemoveAddCycle : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 10;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			for (int32 i = 0; i < NumEntities; ++i)
			{
				const FMassEntityHandle Entity = Entities[i];

				// First Add: Set initial value
				{
					FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
					AITEST_TRUE(TEXT("First add succeeded"), View.IsValid());

					FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
					Fragment->Value = 100 + i;
				}

				// Verify first add
				{
					FMassEntityView EntityView(*EntityManager, Entity);
					const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
					AITEST_NOT_NULL(TEXT("Fragment exists after first add"), Fragment);
					AITEST_EQUAL(TEXT("First add value correct"), Fragment->Value, 100 + i);
				}

				// Remove
				EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				
				// Verify removal
				{
					FMassEntityView EntityView(*EntityManager, Entity);
					const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
					AITEST_NULL(TEXT("Fragment removed"), Fragment);
				}

				// Second Add: Set different value to ensure fresh memory
				{
					FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
					AITEST_TRUE(TEXT("Second add succeeded"), View.IsValid());

					FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
					// Value should be default-initialized (0), not the old value (100 + i)
					AITEST_EQUAL(TEXT("Second add has fresh default value, not old data"), Fragment->Value, 0);

					// Set new value
					Fragment->Value = 200 + i;
				}

				// Verify second add has new value
				{
					FMassEntityView EntityView(*EntityManager, Entity);
					const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
					AITEST_NOT_NULL(TEXT("Fragment exists after second add"), Fragment);
					AITEST_EQUAL(TEXT("Second add value correct"), Fragment->Value, 200 + i);
					AITEST_NOT_EQUAL(TEXT("Second add value different from first add"), Fragment->Value, 100 + i);
				}
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_AddRemoveAddCycle, "System.Mass.Stress.Sparse.AddRemoveAddCycle");

	struct FSparseElements_AddRemoveAddAtChunkBoundaries : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			// Test at specific indices that cross chunk boundaries (default 128 per chunk)
			constexpr int32 BoundaryIndices[] = { 0, 1, 126, 127, 128, 129, 255, 256, 383, 384 };
			constexpr int32 MaxIndex = 500;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, MaxIndex, Entities);

			for (int32 BoundaryIdx : BoundaryIndices)
			{
				if (BoundaryIdx >= Entities.Num())
					continue;

				const FMassEntityHandle Entity = Entities[BoundaryIdx];

				// Cycle 1: Add with value A
				{
					FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
					AITEST_TRUE(TEXT("Boundary add 1 succeeded"), View.IsValid());

					FTestFragment_SparseFloat* Fragment = View.GetPtr<FTestFragment_SparseFloat>();
					Fragment->Value = static_cast<float>((BoundaryIdx + 1) * 10);
				}

				// Verify value A
				{
					FConstStructView View = EntityManager->GetSparseElementDataForEntity(
						FTestFragment_SparseFloat::StaticStruct(), Entity);
					AITEST_TRUE(TEXT("Fragment accessible at boundary"), View.IsValid());
					AITEST_EQUAL(TEXT("Boundary value A correct"),
						View.Get<FTestFragment_SparseFloat>().Value, static_cast<float>((BoundaryIdx + 1) * 10));
				}

				// Remove at boundary
				EntityManager->RemoveSparseElementFromEntity(
					Entity, FTestFragment_SparseFloat::StaticStruct());

				// Verify removed
				{
					FConstStructView View = EntityManager->GetSparseElementDataForEntity(
						FTestFragment_SparseFloat::StaticStruct(), Entity);
					AITEST_FALSE(TEXT("Fragment removed at boundary"), View.IsValid());
				}

				// Cycle 2: Add with value B
				{
					FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
					AITEST_TRUE(TEXT("Boundary add 2 succeeded"), View.IsValid());

					FTestFragment_SparseFloat* Fragment = View.GetPtr<FTestFragment_SparseFloat>();
					// Should be default-initialized
					AITEST_EQUAL(TEXT("Boundary second add has fresh default value"), Fragment->Value, 0.0f);

					Fragment->Value = static_cast<float>((BoundaryIdx + 1) * 20); // Different value
				}

				// Verify value B (different from value A)
				{
					FConstStructView View = EntityManager->GetSparseElementDataForEntity(
						FTestFragment_SparseFloat::StaticStruct(), Entity);
					AITEST_TRUE(TEXT("Fragment accessible after second add"), View.IsValid());
					AITEST_EQUAL(TEXT("Boundary value B correct"),
						View.Get<FTestFragment_SparseFloat>().Value, static_cast<float>((BoundaryIdx + 1) * 20));
					AITEST_NOT_EQUAL(TEXT("Boundary value B different from A"),
						View.Get<FTestFragment_SparseFloat>().Value, static_cast<float>((BoundaryIdx + 1) * 10));
				}
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_AddRemoveAddAtChunkBoundaries, "System.Mass.Stress.Sparse.AddRemoveAddBoundaries");

	struct FSparseElements_AddRemoveAddWithTags : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 5;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

			for (const FMassEntityHandle& Entity : Entities)
			{
				// Add sparse tag
				{
					FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
					// Tags return invalid view (no data)
					AITEST_FALSE(TEXT("Tag add returns empty view"), View.IsValid());
				}

				// Verify tag present
				{
					FMassEntityView EntityView(*EntityManager, Entity);
					AITEST_TRUE(TEXT("Tag present after add"), EntityView.HasElement<FTestTag_SparseA>());
				}

				// Remove tag
				EntityManager->RemoveSparseElementFromEntity(Entity, FTestTag_SparseA::StaticStruct());

				// Verify tag removed
				{
					FMassEntityView EntityView(*EntityManager, Entity);
					AITEST_FALSE(TEXT("Tag removed"), EntityView.HasElement<FTestTag_SparseA>());
				}

				// Add tag again
				{
					FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
					AITEST_FALSE(TEXT("Tag second add returns empty view"), View.IsValid());
				}

				// Verify tag present again
				{
					FMassEntityView EntityView(*EntityManager, Entity);
					AITEST_TRUE(TEXT("Tag present after second add"), EntityView.HasElement<FTestTag_SparseA>());
				}
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_AddRemoveAddWithTags, "System.Mass.Stress.Sparse.AddRemoveAddTags");

	struct FSparseElements_AddRemoveAddMultipleTypes : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

			// Add two different sparse types
			{
				FStructView IntView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				IntView.Get<FTestFragment_SparseInt>().Value = 100;

				FStructView FloatView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
				FloatView.Get<FTestFragment_SparseFloat>().Value = 50.0f;
			}

			// Verify both present
			{
				FMassEntityView EntityView(*EntityManager, Entity);
				AITEST_TRUE(TEXT("Has SparseInt"), EntityView.HasElement<FTestFragment_SparseInt>());
				AITEST_TRUE(TEXT("Has SparseFloat"), EntityView.HasElement<FTestFragment_SparseFloat>());
			}

			// Remove only SparseInt
			EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

			// Verify SparseInt removed but SparseFloat remains
			{
				FMassEntityView EntityView(*EntityManager, Entity);
				AITEST_FALSE(TEXT("SparseInt removed"), EntityView.HasElement<FTestFragment_SparseInt>());
				AITEST_TRUE(TEXT("SparseFloat still present"), EntityView.HasElement<FTestFragment_SparseFloat>());

				const FTestFragment_SparseFloat* FloatFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();
				AITEST_NOT_NULL(TEXT("SparseFloat accessible"), FloatFragment);
				AITEST_EQUAL(TEXT("SparseFloat value unchanged"), FloatFragment->Value, 50.0f);
			}

			// Add SparseInt again with different value
			{
				FStructView IntView = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_TRUE(TEXT("SparseInt re-added"), IntView.IsValid());

				FTestFragment_SparseInt* IntFragment = IntView.GetPtr<FTestFragment_SparseInt>();
				AITEST_EQUAL(TEXT("Re-added SparseInt has fresh default value"), IntFragment->Value, 0);

				IntFragment->Value = 200; // Different from original 100
			}

			// Verify both present with correct values
			{
				FMassEntityView EntityView(*EntityManager, Entity);
				AITEST_TRUE(TEXT("Both types present"),
					EntityView.HasElement<FTestFragment_SparseInt>() &&
					EntityView.HasElement<FTestFragment_SparseFloat>());

				const FTestFragment_SparseInt* IntFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				const FTestFragment_SparseFloat* FloatFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

				AITEST_NOT_NULL(TEXT("SparseInt accessible"), IntFragment);
				AITEST_NOT_NULL(TEXT("SparseFloat accessible"), FloatFragment);

				AITEST_EQUAL(TEXT("SparseInt has new value"), IntFragment->Value, 200);
				AITEST_NOT_EQUAL(TEXT("SparseInt different from original"), IntFragment->Value, 100);
				AITEST_EQUAL(TEXT("SparseFloat value unchanged"), FloatFragment->Value, 50.0f);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_AddRemoveAddMultipleTypes, "System.Mass.Stress.Sparse.AddRemoveAddMultipleTypes");

	struct FSparseElements_AddRemoveAddWithArchetypeChange : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

			// Add sparse element to entity in IntsArchetype
			{
				FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				View.Get<FTestFragment_SparseInt>().Value = 111;
			}

			// Verify
			{
				AITEST_EQUAL(TEXT("Entity in IntsArchetype"),
					EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

				FMassEntityView EntityView(*EntityManager, Entity);
				const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				AITEST_NOT_NULL(TEXT("Sparse element present"), Fragment);
				AITEST_EQUAL(TEXT("Value correct"), Fragment->Value, 111);
			}

			// Remove sparse element
			{
				EntityManager->RemoveSparseElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
			}

			// Change archetype by adding regular fragment
			{
				EntityManager->AddFragmentToEntity(Entity, FTestFragment_Float::StaticStruct());
				AITEST_EQUAL(TEXT("Entity moved to FloatsIntsArchetype"),
					EntityManager->GetArchetypeForEntity(Entity), FloatsIntsArchetype);
			}

			// Add sparse element again (now in different archetype)
			{
				FStructView View = EntityManager->AddSparseElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
				AITEST_TRUE(TEXT("Sparse element added after archetype change"), View.IsValid());

				FTestFragment_SparseInt* Fragment = View.GetPtr<FTestFragment_SparseInt>();
				AITEST_EQUAL(TEXT("Fresh default value after archetype change"), Fragment->Value, 0);

				Fragment->Value = 222;
			}

			// Verify in new archetype
			{
				AITEST_EQUAL(TEXT("Still in FloatsIntsArchetype"),
					EntityManager->GetArchetypeForEntity(Entity), FloatsIntsArchetype);

				FMassEntityView EntityView(*EntityManager, Entity);
				const FTestFragment_SparseInt* Fragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				AITEST_NOT_NULL(TEXT("Sparse element present in new archetype"), Fragment);
				AITEST_EQUAL(TEXT("New value correct"), Fragment->Value, 222);
				AITEST_NOT_EQUAL(TEXT("Value different from original"), Fragment->Value, 111);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_AddRemoveAddWithArchetypeChange, "System.Mass.Stress.Sparse.AddRemoveAddWithArchetypeChange");

	struct FSparseElements_BatchAddFragmentInstances_PureSparse : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 20;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Create payload with sparse fragment instances
			UE::Mass::TMultiArray<FTestFragment_SparseInt, FTestFragment_SparseFloat> Payload;
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FTestFragment_SparseInt SparseInt;
				SparseInt.Value = i * 100;

				FTestFragment_SparseFloat SparseFloat;
				SparseFloat.Value = static_cast<float>(i * 5);

				Payload.Add(SparseInt, SparseFloat);
			}

			// Create entity collections with payload
			TArray<FStructArrayView> GenericPayload;
			GenericPayload.Reserve(Payload.GetNumArrays());
			Payload.GetAsGenericMultiArray(GenericPayload);

			TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
			FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
				*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
				FMassGenericPayloadView(GenericPayload), EntityCollections);

			// Build fragments bitset
			FMassFragmentBitSet FragmentsAffected;
			FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());
			FragmentsAffected.Add(FTestFragment_SparseFloat::StaticStruct());

			// Execute batch add
			EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

			// Verify all entities have sparse fragments with correct data
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);

				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				const FTestFragment_SparseFloat* SparseFloat = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

				AITEST_NOT_NULL(TEXT("SparseInt added"), SparseInt);
				AITEST_NOT_NULL(TEXT("SparseFloat added"), SparseFloat);

				AITEST_EQUAL(TEXT("SparseInt value correct"), SparseInt->Value, i * 100);
				AITEST_EQUAL(TEXT("SparseFloat value correct"), SparseFloat->Value, static_cast<float>(i * 5));
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchAddFragmentInstances_PureSparse, "System.Mass.Stress.Sparse.BatchAddFragmentInstances.PureSparse");

	struct FSparseElements_BatchAddFragmentInstances_MixedRegularAndSparse : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 15;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Create payload with both regular and sparse fragments
			UE::Mass::TMultiArray<FTestFragment_Float, FTestFragment_SparseInt> Payload;
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FTestFragment_Float RegularFloat;
				RegularFloat.Value = static_cast<float>(i * 10);

				FTestFragment_SparseInt SparseInt;
				SparseInt.Value = i * 200;

				Payload.Add(RegularFloat, SparseInt);
			}

			// Create entity collections with payload
			TArray<FStructArrayView> GenericPayload;
			GenericPayload.Reserve(Payload.GetNumArrays());
			Payload.GetAsGenericMultiArray(GenericPayload);

			TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
			FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
				*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
				FMassGenericPayloadView(GenericPayload), EntityCollections);

			// Build fragments bitset (mixed regular + sparse)
			FMassFragmentBitSet FragmentsAffected;
			FragmentsAffected.Add(FTestFragment_Float::StaticStruct());
			FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());

			// Execute batch add
			EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

			// Verify all entities have both regular and sparse fragments
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);

				// Regular fragment should be in archetype
				const FTestFragment_Float* RegularFloat = EntityView.GetFragmentDataPtr<FTestFragment_Float>();
				AITEST_NOT_NULL(TEXT("Regular float added"), RegularFloat);
				AITEST_EQUAL(TEXT("Regular float value correct"), RegularFloat->Value, static_cast<float>(i * 10));

				// Sparse fragment should be in external storage
				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				AITEST_NOT_NULL(TEXT("Sparse int added"), SparseInt);
				AITEST_EQUAL(TEXT("Sparse int value correct"), SparseInt->Value, i * 200);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchAddFragmentInstances_MixedRegularAndSparse, "System.Mass.Stress.Sparse.BatchAddFragmentInstances.MixedRegularAndSparse");

	struct FSparseElements_BatchAddFragmentInstances_CrossChunks : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(FloatsArchetype);
			const int32 NumEntities = static_cast<int32>(static_cast<float>(EntitiesPerChunk) * 2.5f); // Span 3 chunks

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

			// Create payload spanning multiple chunks
			UE::Mass::TMultiArray<FTestFragment_SparseInt> Payload;
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FTestFragment_SparseInt SparseInt;
				SparseInt.Value = i * 777;
				Payload.Add(SparseInt);
			}

			// Create entity collections
			TArray<FStructArrayView> GenericPayload;
			GenericPayload.Reserve(Payload.GetNumArrays());
			Payload.GetAsGenericMultiArray(GenericPayload);

			TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
			FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
				*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
				FMassGenericPayloadView(GenericPayload), EntityCollections);

			FMassFragmentBitSet FragmentsAffected;
			FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());

			// Execute across chunks
			EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

			// Verify all entities across all chunks have correct data
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);
				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

				AITEST_NOT_NULL(TEXT("Sparse fragment present across chunks"), SparseInt);
				AITEST_EQUAL(TEXT("Sparse fragment value correct across chunks"), SparseInt->Value, i * 777);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchAddFragmentInstances_CrossChunks, "System.Mass.Stress.Sparse.BatchAddFragmentInstances.CrossChunks");

	struct FSparseElements_BatchAddFragmentInstances_OverwriteExisting : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 10;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// First, add sparse fragments with initial values
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FStructView SparseInt = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
				SparseInt.Get<FTestFragment_SparseInt>().Value = (i + 1) * 10; // Initial value
			}

			// Verify initial values
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);
				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				AITEST_NOT_NULL(TEXT("Initial sparse fragment present"), SparseInt);
				AITEST_EQUAL(TEXT("Initial value correct"), SparseInt->Value, (i + 1) * 10);
			}

			// Now use BatchAddFragmentInstancesForEntities to overwrite with new values
			UE::Mass::TMultiArray<FTestFragment_SparseInt> Payload;
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FTestFragment_SparseInt NewSparseInt;
				NewSparseInt.Value = (i + 1) * 999; // New value
				Payload.Add(NewSparseInt);
			}

			TArray<FStructArrayView> GenericPayload;
			GenericPayload.Reserve(Payload.GetNumArrays());
			Payload.GetAsGenericMultiArray(GenericPayload);

			TArray<FMassArchetypeEntityCollectionWithPayload> EntityCollections;
			FMassArchetypeEntityCollectionWithPayload::CreateEntityRangesWithPayload(
				*EntityManager, Entities, FMassArchetypeEntityCollection::FoldDuplicates,
				FMassGenericPayloadView(GenericPayload), EntityCollections);

			FMassFragmentBitSet FragmentsAffected;
			FragmentsAffected.Add(FTestFragment_SparseInt::StaticStruct());

			EntityManager->BatchAddFragmentInstancesForEntities(EntityCollections, FragmentsAffected);

			// Verify values were overwritten
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);
				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

				AITEST_NOT_NULL(TEXT("Sparse fragment still present"), SparseInt);
				AITEST_EQUAL(TEXT("Value overwritten correctly"), SparseInt->Value, (i + 1) * 999);
				AITEST_NOT_EQUAL(TEXT("Value changed from initial"), SparseInt->Value, (i + 1) * 10);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchAddFragmentInstances_OverwriteExisting, "System.Mass.Stress.Sparse.BatchAddFragmentInstances.OverwriteExisting");

	struct FSparseElements_CommandAddFragmentInstances_PureSparse : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 12;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

			// Use command to add sparse fragments
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FTestFragment_SparseInt SparseInt;
				SparseInt.Value = i * 50;

				FTestFragment_SparseFloat SparseFloat;
				SparseFloat.Value = static_cast<float>(i * 3);

				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], SparseInt, SparseFloat);
			}

			// Flush commands
			EntityManager->FlushCommands();

			// Verify all entities have sparse fragments with correct values
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);

				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
				const FTestFragment_SparseFloat* SparseFloat = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

				AITEST_NOT_NULL(TEXT("Command added SparseInt"), SparseInt);
				AITEST_NOT_NULL(TEXT("Command added SparseFloat"), SparseFloat);

				AITEST_EQUAL(TEXT("Command SparseInt value correct"), SparseInt->Value, i * 50);
				AITEST_EQUAL(TEXT("Command SparseFloat value correct"), SparseFloat->Value, static_cast<float>(i * 3));
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_CommandAddFragmentInstances_PureSparse, "System.Mass.Stress.Sparse.CommandAddFragmentInstances.PureSparse");

	struct FSparseElements_CommandAddFragmentInstances_MixedRegularAndSparse : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 8;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Use command with mixed regular + sparse fragments
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FTestFragment_Float RegularFloat;
				RegularFloat.Value = static_cast<float>(i * 25);

				FTestFragment_SparseInt SparseInt;
				SparseInt.Value = i * 888;

				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], RegularFloat, SparseInt);
			}

			EntityManager->FlushCommands();

			// Verify both regular and sparse fragments
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);

				const FTestFragment_Float* RegularFloat = EntityView.GetFragmentDataPtr<FTestFragment_Float>();
				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

				AITEST_NOT_NULL(TEXT("Command added regular float"), RegularFloat);
				AITEST_NOT_NULL(TEXT("Command added sparse int"), SparseInt);

				AITEST_EQUAL(TEXT("Command regular float value correct"), RegularFloat->Value, static_cast<float>(i * 25));
				AITEST_EQUAL(TEXT("Command sparse int value correct"), SparseInt->Value, i * 888);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_CommandAddFragmentInstances_MixedRegularAndSparse, "System.Mass.Stress.Sparse.CommandAddFragmentInstances.MixedRegularAndSparse");

	struct FSparseElements_CommandAddFragmentInstances_CrossChunks : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
			const int32 NumChunks = 4;
			const int32 NumEntities = EntitiesPerChunk * NumChunks;

			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

			// Use command across multiple chunks
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FTestFragment_SparseFloat SparseFloat;
				SparseFloat.Value = static_cast<float>(i * 11);
				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], SparseFloat);
			}

			EntityManager->FlushCommands();

			// Verify across all chunks
			int32 ChunksFound = 0;
			int32 LastChunkIndex = -1;

			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);
				const FTestFragment_SparseFloat* SparseFloat = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseFloat>();

				AITEST_NOT_NULL(TEXT("Sparse fragment present across chunks via command"), SparseFloat);
				AITEST_EQUAL(TEXT("Value correct across chunks via command"), SparseFloat->Value, static_cast<float>(i * 11));

				// Track chunk transitions
				int32 CurrentChunkIndex = i / EntitiesPerChunk;
				if (CurrentChunkIndex != LastChunkIndex)
				{
					++ChunksFound;
					LastChunkIndex = CurrentChunkIndex;
				}
			}

			AITEST_EQUAL(TEXT("Command processed all chunks"), ChunksFound, NumChunks);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_CommandAddFragmentInstances_CrossChunks, "System.Mass.Stress.Sparse.CommandAddFragmentInstances.CrossChunks");

	struct FSparseElements_CommandAddFragmentInstances_PartialBatch : FEntityTestBase
	{
		virtual bool InstantTest() override
		{
			constexpr int32 NumEntities = 30;
			TArray<FMassEntityHandle> Entities;
			EntityManager->BatchCreateEntities(FloatsArchetype, NumEntities, Entities);

			// Only add sparse fragments to every 3rd entity via command
			int32 ExpectedCount = 0;
			for (int32 i = 0; i < NumEntities; i += 3)
			{
				FTestFragment_SparseInt SparseInt;
				SparseInt.Value = i * 33;
				EntityManager->Defer().PushCommand<FMassCommandAddFragmentInstances>(Entities[i], SparseInt);
				++ExpectedCount;
			}

			EntityManager->FlushCommands();

			// Verify only targeted entities have sparse fragments
			int32 ActualCount = 0;
			for (int32 i = 0; i < NumEntities; ++i)
			{
				FMassEntityView EntityView(*EntityManager, Entities[i]);
				const FTestFragment_SparseInt* SparseInt = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();

				if (i % 3 == 0)
				{
					AITEST_NOT_NULL(TEXT("Targeted entity has sparse fragment"), SparseInt);
					AITEST_EQUAL(TEXT("Targeted entity value correct"), SparseInt->Value, i * 33);
					++ActualCount;
				}
				else
				{
					AITEST_NULL(TEXT("Non-targeted entity has no sparse fragment"), SparseInt);
				}
			}

			AITEST_EQUAL(TEXT("Correct number of entities received sparse fragments"), ActualCount, ExpectedCount);

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FSparseElements_CommandAddFragmentInstances_PartialBatch, "System.Mass.Stress.Sparse.CommandAddFragmentInstances.PartialBatch");

	//struct FSparseElements_Storage_Statistics : FEntityTestBase
	//{
	//	virtual bool InstantTest() override
	//	{
	//		FSparseElementsStorage Storage;

	//		constexpr int32 NumEntities = 200;
	//		TArray<FMassEntityHandle> Entities;
	//		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

	//		// Add elements across multiple chunks (default 128/chunk)
	//		for (int32 i = 0; i < NumEntities; i += 2)  // Every other entity
	//		{
	//			Storage.AddElementToEntity<FTestFragment_SparseInt>(Entities[i]);
	//		}

	//		// Get type stats
	//		auto Stats = Storage.GetTypeStats<FTestFragment_SparseInt>();
	//		AITEST_EQUAL(TEXT("Stats reports correct count"), Stats.NumElements, 100u);
	//		AITEST_TRUE(TEXT("Stats reports multiple chunks"), Stats.NumChunks >= 2);
	//		AITEST_TRUE(TEXT("Occupation rate is reasonable"), Stats.OccupationRate > 0.0f && Stats.OccupationRate < 1.0f);
	//		AITEST_TRUE(TEXT("Memory usage reported"), Stats.MemoryUsed > 0);

	//		// Get storage stats
	//		auto StorageStats = Storage.GetStorageStats();
	//		AITEST_EQUAL(TEXT("Storage stats total elements"), StorageStats.TotalElements, 100u);
	//		AITEST_TRUE(TEXT("Storage stats active pools"), StorageStats.NumActiveTypePools >= 1);

	//		return true;
	//	}
	//};
	//IMPLEMENT_AI_INSTANT_TEST(FSparseElements_Storage_CountTracking, "System.Mass.Stress.Sparse.StorageStatistics");

#endif // WITH_MASSENTITY_DEBUG

//-----------------------------------------------------------------------------
// Chunk and Archetype Bitset Tracking Tests
//-----------------------------------------------------------------------------
#if WITH_MASSENTITY_DEBUG
struct FSparseElements_ChunkBitset_AddSingle : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Add sparse element to first entity
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

		// Verify chunk bitset is updated
		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		AITEST_TRUE("Chunk should have sparse elements", Chunk.HasSparseElements());

		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();
		AITEST_TRUE("Chunk bitset should contain sparse int",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ChunkBitset_AddSingle, "System.Mass.Stress.Sparse.ChunkBitset.AddSingle");

struct FSparseElements_ChunkBitset_RemoveSingle : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Add sparse element to first two entities
		FStructView _[] = {
			EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct())
			, EntityManager->AddSparseElementToEntity(Entities[1], FTestFragment_SparseInt::StaticStruct())
		};

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

		AITEST_TRUE("Chunk bitset should contain sparse int after adds",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Remove from one entity - should still be in chunk bitset
		EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
		AITEST_TRUE("Chunk bitset should still contain sparse int (one entity remains)",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Remove from last entity - should be removed from chunk bitset
		EntityManager->RemoveSparseElementFromEntity(Entities[1], FTestFragment_SparseInt::StaticStruct());
		AITEST_FALSE("Chunk bitset should NOT contain sparse int after all removed",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ChunkBitset_RemoveSingle, "System.Mass.Stress.Sparse.ChunkBitset.RemoveSingle");

struct FSparseElements_ChunkBitset_BatchAdd : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 50;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Batch add sparse elements to first 20 entities
		TArray<FMassEntityHandle> BatchEntities;
		for (int32 i = 0; i < 20; ++i)
		{
			BatchEntities.Add(Entities[i]);
		}
		FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, BatchEntities, FMassArchetypeEntityCollection::NoDuplicates);
		EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());

		// Verify chunk bitset is updated
		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

		AITEST_TRUE("Chunk bitset should contain sparse int after batch add",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ChunkBitset_BatchAdd, "System.Mass.Stress.Sparse.ChunkBitset.BatchAdd");

struct FSparseElements_ChunkBitset_BatchRemove : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 50;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Batch add to all entities
		{
			FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, Entities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
		}

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

		AITEST_TRUE("Chunk bitset should contain sparse int",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Batch remove from half the entities - should still be in chunk bitset
		TArray<FMassEntityHandle> HalfEntities;
		for (int32 i = 0; i < 25; ++i)
		{
			HalfEntities.Add(Entities[i]);
		}
		{
			FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, HalfEntities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
		}

		AITEST_TRUE("Chunk bitset should still contain sparse int (some entities remain)",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Batch remove from remaining entities - should be removed from chunk bitset
		TArray<FMassEntityHandle> RemainingEntities;
		for (int32 i = 25; i < NumEntities; ++i)
		{
			RemainingEntities.Add(Entities[i]);
		}
		{
			FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, RemainingEntities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
		}

		AITEST_FALSE("Chunk bitset should NOT contain sparse int after all removed",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ChunkBitset_BatchRemove, "System.Mass.Stress.Sparse.ChunkBitset.BatchRemove");

struct FSparseElements_ChunkBitset_MultipleSparseTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 20;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Add different sparse types to different entities
		FStructView _[] = {
			EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct())
			, EntityManager->AddSparseElementToEntity(Entities[1], FTestFragment_SparseFloat::StaticStruct())
			, EntityManager->AddSparseElementToEntity(Entities[2], FTestTag_SparseA::StaticStruct())
		};

		// Verify chunk bitset contains all three types
		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

		AITEST_TRUE("Chunk bitset should contain sparse int",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Chunk bitset should contain sparse float",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_TRUE("Chunk bitset should contain sparse tag A",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestTag_SparseA::StaticStruct()));

		// Remove one type completely
		EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

		AITEST_FALSE("Chunk bitset should NOT contain sparse int after removal",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Chunk bitset should still contain sparse float",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseFloat::StaticStruct()));
		AITEST_TRUE("Chunk bitset should still contain sparse tag A",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ChunkBitset_MultipleSparseTypes, "System.Mass.Stress.Sparse.ChunkBitset.MultipleSparseTypes");

struct FSparseElements_ChunkBitset_CrossChunk : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const int32 NumEntitiesPerChunk = Archetype.GetNumEntitiesPerChunk();
		int32 NumEntities = NumEntitiesPerChunk + 1;

		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Add sparse element to entities in first chunk only
		for (int32 i = 0; i < 64; ++i)
		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
		}

		// Verify first chunk has the sparse element
		const FMassArchetypeChunk& Chunk0 = Archetype.DebugGetChunk(0);
		AITEST_TRUE("First chunk should have sparse elements", Chunk0.HasSparseElements());
		AITEST_TRUE("First chunk bitset should contain sparse int",
			Chunk0.GetSparseElementsUnsafe().GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Verify second chunk does NOT have the sparse element
		const FMassArchetypeChunk& Chunk1 = Archetype.DebugGetChunk(1);
		AITEST_FALSE("Second chunk should not have sparse int in bitset",
			Chunk1.HasSparseElements() && Chunk1.GetSparseElementsUnsafe().GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Add sparse element to second chunk
		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[NumEntitiesPerChunk], FTestFragment_SparseInt::StaticStruct());
		}
		AITEST_TRUE("Second chunk should now have sparse elements", Chunk1.HasSparseElements());
		AITEST_TRUE("Second chunk bitset should now contain sparse int",
			Chunk1.GetSparseElementsUnsafe().GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ChunkBitset_CrossChunk, "System.Mass.Stress.Sparse.ChunkBitset.CrossChunk");
#endif // WITH_MASSENTITY_DEBUG

struct FSparseElements_ArchetypeBitset_SingleChunk : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);

		// Initially, archetype should not contain sparse elements
		AITEST_FALSE("Archetype should not contain sparse int initially",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

		// Add sparse element
		FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

		// Archetype bitset should now contain the sparse element
		AITEST_TRUE("Archetype should contain sparse int after add",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

		// Remove the sparse element from all entities
		EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());

		// Archetype bitset should no longer contain the sparse element
		AITEST_FALSE("Archetype should not contain sparse int after removal",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ArchetypeBitset_SingleChunk, "System.Mass.Stress.Sparse.ArchetypeBitset.SingleChunk");

#if WITH_MASSENTITY_DEBUG
struct FSparseElements_ArchetypeBitset_MultipleChunks : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		const int32 EntitiesPerChunk = EntityManager->DebugGetArchetypeEntitiesCountPerChunk(IntsArchetype);
		const int32 NumEntities = EntitiesPerChunk + 1;  // Multiple chunks
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);

		// Add sparse element to entities in different chunks
		FStructView _ [] = {
			EntityManager->AddSparseElementToEntity(Entities[10], FTestFragment_SparseInt::StaticStruct())   // Chunk 0
			, EntityManager->AddSparseElementToEntity(Entities[EntitiesPerChunk], FTestFragment_SparseFloat::StaticStruct()) // Chunk 1
		};

		// Archetype should contain both types
		AITEST_TRUE("Archetype should contain sparse int",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Archetype should contain sparse float",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

		// Remove sparse int from first chunk
		EntityManager->RemoveSparseElementFromEntity(Entities[10], FTestFragment_SparseInt::StaticStruct());

		AITEST_FALSE("Archetype should not contain sparse int after removal",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Archetype should still contain sparse float",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ArchetypeBitset_MultipleChunks, "System.Mass.Stress.Sparse.ArchetypeBitset.MultipleChunks");

struct FSparseElements_EntityRemoval_ChunkBitsetUpdate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 20;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Add sparse elements to first 5 entities
		for (int32 i = 0; i < 5; ++i)
		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
		}

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

		AITEST_TRUE("Chunk should contain sparse int",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Destroy entities with sparse elements one by one (but not all)
		for (int32 i = 0; i < 3; ++i)
		{
			EntityManager->DestroyEntity(Entities[i]);
		}
		EntityManager->FlushCommands();

		// Chunk should still contain sparse int (2 entities remain)
		AITEST_TRUE("Chunk should still contain sparse int after partial removal",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Destroy remaining entities with sparse elements
		for (int32 i = 3; i < 5; ++i)
		{
			EntityManager->DestroyEntity(Entities[i]);
		}
		EntityManager->FlushCommands();

		// Chunk should no longer contain sparse int
		AITEST_FALSE("Chunk should not contain sparse int after all removed",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_EntityRemoval_ChunkBitsetUpdate, "System.Mass.Stress.Sparse.EntityRemoval.ChunkBitsetUpdate");

struct FSparseElements_BatchEntityDestruction_ChunkBitsetUpdate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 50;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Add sparse elements to all entities
		{
			FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, Entities, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
		}

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

		AITEST_TRUE("Chunk should contain sparse int",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Batch destroy all entities
		EntityManager->BatchDestroyEntities(Entities);
		EntityManager->FlushCommands();

		// Chunk should no longer exist or be empty
		AITEST_EQUAL("Archetype should have no entities after batch destruction", Archetype.GetNumEntities(), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_BatchEntityDestruction_ChunkBitsetUpdate, "System.Mass.Stress.Sparse.EntityRemoval.BatchDestruction");
#endif // WITH_MASSENTITY_DEBUG

struct FSparseElements_ArchetypeChange_BitsetTracking : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		// Create source archetype with just Int
		FMassArchetypeCompositionDescriptor SourceComposition;
		SourceComposition.Add<FTestFragment_Int>();
		const FMassArchetypeHandle SourceArchetype = EntityManager->CreateArchetype(SourceComposition);

		// Create target archetype with Int + Float
		FMassArchetypeCompositionDescriptor TargetComposition;
		TargetComposition.Add<FTestFragment_Int>();
		TargetComposition.Add<FTestFragment_Float>();
		const FMassArchetypeHandle TargetArchetype = EntityManager->CreateArchetype(TargetComposition);

		constexpr int32 NumEntities = 20;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(SourceArchetype, NumEntities, Entities);

		// Add sparse elements to entities in source archetype
		for (int32 i = 0; i < 10; ++i)
		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
		}

		FMassArchetypeData& SourceArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(SourceArchetype);
		AITEST_TRUE("Source archetype should contain sparse int",
			SourceArchetypeData.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

		// Move entities to target archetype
		for (int32 i = 0; i < 10; ++i)
		{
			EntityManager->AddFragmentToEntity(Entities[i], FTestFragment_Float::StaticStruct());
		}

		FMassArchetypeData& TargetArchetypeData = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(TargetArchetype);

		// Target archetype should now contain sparse elements
		AITEST_TRUE("Target archetype should contain sparse int after move",
			TargetArchetypeData.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

		// Source archetype should still have entities with sparse elements (remaining 10 entities)
		AITEST_TRUE("Source archetype should no longer contain sparse int - all sparse fragment owning entities have been moved",
			SourceArchetypeData.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()) == false);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ArchetypeChange_BitsetTracking, "System.Mass.Stress.Sparse.ArchetypeChange.BitsetTracking");

struct FSparseElements_ComplexScenario_BitsetAccuracy : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 100;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);

		// Complex sequence of operations
		// 1. Batch add sparse int to first 50
		TArray<FMassEntityHandle> FirstHalf;
		for (int32 i = 0; i < 50; ++i)
		{
			FirstHalf.Add(Entities[i]);
		}
		{
			FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, FirstHalf, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchAddSparseElementToEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
		}
		AITEST_TRUE("Should contain sparse int after batch add",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

		// 2. Add sparse float to entities 25-75 (overlapping)
		for (int32 i = 25; i < 75; ++i)
		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[i], FTestFragment_SparseFloat::StaticStruct());
		}
		AITEST_TRUE("Should contain sparse float",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

		// 3. Remove sparse int from 0-25
		TArray<FMassEntityHandle> FirstQuarter;
		for (int32 i = 0; i < 25; ++i)
		{
			FirstQuarter.Add(Entities[i]);
		}
		{
			FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, FirstQuarter, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
		}
		AITEST_TRUE("Should still contain sparse int (entities 25-49 remain)",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));

		// 4. Destroy entities 40-60
		for (int32 i = 40; i < 60; ++i)
		{
			EntityManager->DestroyEntity(Entities[i]);
		}
		EntityManager->FlushCommands();

		// Both types should still be present
		AITEST_TRUE("Should still contain sparse int (entities 25-39 remain)",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Should still contain sparse float (entities 25-39 and 60-74 remain)",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

		// 5. Remove all remaining sparse int
		TArray<FMassEntityHandle> RemainingWithInt;
		for (int32 i = 25; i < 40; ++i)
		{
			RemainingWithInt.Add(Entities[i]);
		}
		{
			FMassArchetypeEntityCollection Collection = FMassArchetypeEntityCollection(IntsArchetype, RemainingWithInt, FMassArchetypeEntityCollection::NoDuplicates);
			EntityManager->BatchRemoveSparseElementFromEntities({Collection}, FTestFragment_SparseInt::StaticStruct());
		}

		AITEST_FALSE("Should NOT contain sparse int after all removed",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Should still contain sparse float",
			Archetype.DoesContainEntitiesWithSparseElement(FTestFragment_SparseFloat::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ComplexScenario_BitsetAccuracy, "System.Mass.Stress.Sparse.Complex.BitsetAccuracy");

#if WITH_MASSENTITY_DEBUG
struct FSparseElements_ReAdd_ChunkBitsetConsistency : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		CA_ASSUME(EntityManager);

		constexpr int32 NumEntities = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		FMassArchetypeData& Archetype = FMassArchetypeHelper::ArchetypeDataFromHandleChecked(IntsArchetype);
		const FMassArchetypeChunk& Chunk = Archetype.DebugGetChunk(0);
		const UE::Mass::FChunkSparseElements& ChunkSparseElements = Chunk.GetSparseElementsUnsafe();

		// Add -> Remove -> Add cycle
		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
		}
		AITEST_TRUE("Chunk should contain sparse int after first add",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
		AITEST_FALSE("Chunk should NOT contain sparse int after remove",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
		}
		AITEST_TRUE("Chunk should contain sparse int after re-add",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		// Add to second entity and remove from first - bitset should remain
		{
			FStructView _ = EntityManager->AddSparseElementToEntity(Entities[1], FTestFragment_SparseInt::StaticStruct());
		}
		EntityManager->RemoveSparseElementFromEntity(Entities[0], FTestFragment_SparseInt::StaticStruct());
		AITEST_TRUE("Chunk should still contain sparse int (second entity has it)",
			ChunkSparseElements.GetSparseElementsPresent().Contains(FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_ReAdd_ChunkBitsetConsistency, "System.Mass.Stress.Sparse.ReAdd.ChunkBitsetConsistency");
#endif // WITH_MASSENTITY_DEBUG

//----------------------------------------------------------------------//
// Unified API Tests - verifying Fragment/Tag API works seamlessly with Sparse Elements
//----------------------------------------------------------------------//

struct FSparseElements_UnifiedAPI_AddFragmentToEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Verify entity doesn't have sparse fragment initially
		AITEST_FALSE("Entity should not have sparse fragment initially",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		// Use AddFragmentToEntity with sparse fragment type
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Verify sparse fragment was added
		AITEST_TRUE("Entity should have sparse fragment after AddFragmentToEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		// Verify entity remained in same archetype (sparse doesn't change archetype)
		AITEST_EQUAL("Entity should remain in original archetype",
			EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

		// Verify we can access the sparse fragment data
		FMassEntityView EntityView(*EntityManager, Entity);
		const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Should be able to access sparse fragment data", SparseFragment);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_AddFragmentToEntity, "System.Mass.Stress.Sparse.UnifiedAPI.AddFragmentToEntity");


struct FSparseElements_UnifiedAPI_AddFragmentToEntity_WithInitializer : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 InitialValue = 42;
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Use AddFragmentToEntity with initializer callback for sparse fragment
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct(),
			[InitialValue](void* FragmentMemory, const UScriptStruct& FragmentType)
			{
				FTestFragment_SparseInt* Fragment = static_cast<FTestFragment_SparseInt*>(FragmentMemory);
				Fragment->Value = InitialValue;
			});

		// Verify sparse fragment was added with correct value
		AITEST_TRUE("Entity should have sparse fragment",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		FMassEntityView EntityView(*EntityManager, Entity);
		const FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Should be able to access sparse fragment data", SparseFragment);
		AITEST_EQUAL("Sparse fragment should have initialized value", SparseFragment->Value, InitialValue);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_AddFragmentToEntity_WithInitializer, "System.Mass.Stress.Sparse.UnifiedAPI.AddFragmentToEntity.WithInitializer");


struct FSparseElements_UnifiedAPI_RemoveFragmentFromEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse fragment first
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		AITEST_TRUE("Entity should have sparse fragment after add",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		// Use RemoveFragmentFromEntity with sparse fragment type
		EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Verify sparse fragment was removed
		AITEST_FALSE("Entity should not have sparse fragment after RemoveFragmentFromEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		// Verify entity remained in same archetype
		AITEST_EQUAL("Entity should remain in original archetype",
			EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_RemoveFragmentFromEntity, "System.Mass.Stress.Sparse.UnifiedAPI.RemoveFragmentFromEntity");


struct FSparseElements_UnifiedAPI_AddTagToEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Verify entity doesn't have sparse tag initially
		AITEST_FALSE("Entity should not have sparse tag initially",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		// Use AddTagToEntity with sparse tag type
		EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());

		// Verify sparse tag was added
		AITEST_TRUE("Entity should have sparse tag after AddTagToEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		// Verify entity remained in same archetype (sparse doesn't change archetype)
		AITEST_EQUAL("Entity should remain in original archetype",
			EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

		// Verify via FMassEntityView
		FMassEntityView EntityView(*EntityManager, Entity);
		AITEST_TRUE("EntityView should report having sparse tag", EntityView.HasElement<FTestTag_SparseA>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_AddTagToEntity, "System.Mass.Stress.Sparse.UnifiedAPI.AddTagToEntity");


struct FSparseElements_UnifiedAPI_RemoveTagFromEntity : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse tag first
		EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());
		AITEST_TRUE("Entity should have sparse tag after add",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		// Use RemoveTagFromEntity with sparse tag type
		EntityManager->RemoveTagFromEntity(Entity, FTestTag_SparseA::StaticStruct());

		// Verify sparse tag was removed
		AITEST_FALSE("Entity should not have sparse tag after RemoveTagFromEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		// Verify entity remained in same archetype
		AITEST_EQUAL("Entity should remain in original archetype",
			EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_RemoveTagFromEntity, "System.Mass.Stress.Sparse.UnifiedAPI.RemoveTagFromEntity");


struct FSparseElements_UnifiedAPI_AddElementToEntity_SparseFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Use generic AddElementToEntity with sparse fragment type
		EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Verify sparse fragment was added
		AITEST_TRUE("Entity should have sparse fragment after AddElementToEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		// Verify archetype unchanged
		AITEST_EQUAL("Entity should remain in original archetype",
			EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_AddElementToEntity_SparseFragment, "System.Mass.Stress.Sparse.UnifiedAPI.AddElementToEntity.SparseFragment");


struct FSparseElements_UnifiedAPI_AddElementToEntity_SparseTag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Use generic AddElementToEntity with sparse tag type
		EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());

		// Verify sparse tag was added
		AITEST_TRUE("Entity should have sparse tag after AddElementToEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		// Verify archetype unchanged
		AITEST_EQUAL("Entity should remain in original archetype",
			EntityManager->GetArchetypeForEntity(Entity), IntsArchetype);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_AddElementToEntity_SparseTag, "System.Mass.Stress.Sparse.UnifiedAPI.AddElementToEntity.SparseTag");


struct FSparseElements_UnifiedAPI_RemoveElementFromEntity_SparseFragment : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add then remove sparse fragment using generic API
		EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		AITEST_TRUE("Entity should have sparse fragment",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		EntityManager->RemoveElementFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Verify sparse fragment was removed
		AITEST_FALSE("Entity should not have sparse fragment after RemoveElementFromEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_RemoveElementFromEntity_SparseFragment, "System.Mass.Stress.Sparse.UnifiedAPI.RemoveElementFromEntity.SparseFragment");


struct FSparseElements_UnifiedAPI_RemoveElementFromEntity_SparseTag : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add then remove sparse tag using generic API
		EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
		AITEST_TRUE("Entity should have sparse tag",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		EntityManager->RemoveElementFromEntity(Entity, FTestTag_SparseA::StaticStruct());

		// Verify sparse tag was removed
		AITEST_FALSE("Entity should not have sparse tag after RemoveElementFromEntity",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_RemoveElementFromEntity_SparseTag, "System.Mass.Stress.Sparse.UnifiedAPI.RemoveElementFromEntity.SparseTag");


struct FSparseElements_UnifiedAPI_DoesEntityHaveElement_AllSparseTypes : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Verify none present initially
		AITEST_FALSE("Should not have sparse int initially",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("Should not have sparse float initially",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseFloat::StaticStruct()));
		AITEST_FALSE("Should not have sparse tag A initially",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
		AITEST_FALSE("Should not have sparse tag B initially",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));

		// Add all sparse element types
		EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		EntityManager->AddElementToEntity(Entity, FTestFragment_SparseFloat::StaticStruct());
		EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());
		EntityManager->AddElementToEntity(Entity, FTestTag_SparseB::StaticStruct());

		// Verify all present
		AITEST_TRUE("Should have sparse int",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Should have sparse float",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseFloat::StaticStruct()));
		AITEST_TRUE("Should have sparse tag A",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
		AITEST_TRUE("Should have sparse tag B",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseB::StaticStruct()));

		// Also verify template version
		AITEST_TRUE("Template DoesEntityHaveElement should work for sparse int",
			EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entity));
		AITEST_TRUE("Template DoesEntityHaveElement should work for sparse tag",
			EntityManager->DoesEntityHaveElement<FTestTag_SparseA>(Entity));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_DoesEntityHaveElement_AllSparseTypes, "System.Mass.Stress.Sparse.UnifiedAPI.DoesEntityHaveElement.AllTypes");


struct FSparseElements_UnifiedAPI_EntityView_HasElement : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
		FMassEntityView EntityView(*EntityManager, Entity);

		// Verify HasElement returns false initially for sparse elements
		AITEST_FALSE("EntityView should not report sparse fragment initially",
			EntityView.HasElement<FTestFragment_SparseInt>());
		AITEST_FALSE("EntityView should not report sparse tag initially",
			EntityView.HasElement<FTestTag_SparseA>());

		// Add sparse elements
		EntityManager->AddElementToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		EntityManager->AddElementToEntity(Entity, FTestTag_SparseA::StaticStruct());

		// Verify HasElement returns true
		AITEST_TRUE("EntityView should report sparse fragment after add",
			EntityView.HasElement<FTestFragment_SparseInt>());
		AITEST_TRUE("EntityView should report sparse tag after add",
			EntityView.HasElement<FTestTag_SparseA>());

		// Test with EIncludeSparseElements parameter
		AITEST_TRUE("HasElement with IncludeSparseElements::Yes should return true",
			EntityView.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::Yes));
		AITEST_FALSE("HasElement with IncludeSparseElements::No should return false for sparse element",
			EntityView.HasElement<FTestFragment_SparseInt>(UE::Mass::EIncludeSparseElements::No));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_EntityView_HasElement, "System.Mass.Stress.Sparse.UnifiedAPI.EntityView.HasElement");


struct FSparseElements_UnifiedAPI_MixedRegularAndSparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add regular fragment (changes archetype)
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_Float::StaticStruct());
		const FMassArchetypeHandle NewArchetype = EntityManager->GetArchetypeForEntity(Entity);
		AITEST_NOT_EQUAL("Archetype should change after adding regular fragment", NewArchetype, IntsArchetype);

		// Add sparse fragment (should not change archetype)
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		AITEST_EQUAL("Archetype should not change after adding sparse fragment",
			EntityManager->GetArchetypeForEntity(Entity), NewArchetype);

		// Add regular tag (changes archetype)
		EntityManager->AddTagToEntity(Entity, FTestTag_A::StaticStruct());
		const FMassArchetypeHandle ArchetypeWithTag = EntityManager->GetArchetypeForEntity(Entity);
		AITEST_NOT_EQUAL("Archetype should change after adding regular tag", ArchetypeWithTag, NewArchetype);

		// Add sparse tag (should not change archetype)
		EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());
		AITEST_EQUAL("Archetype should not change after adding sparse tag",
			EntityManager->GetArchetypeForEntity(Entity), ArchetypeWithTag);

		// Verify all elements present
		AITEST_TRUE("Should have regular int fragment (from original archetype)",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Int::StaticStruct()));
		AITEST_TRUE("Should have regular float fragment",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
		AITEST_TRUE("Should have sparse int fragment",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		AITEST_TRUE("Should have regular tag A",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));
		AITEST_TRUE("Should have sparse tag A",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		// Remove sparse elements - archetype should not change
		EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		EntityManager->RemoveTagFromEntity(Entity, FTestTag_SparseA::StaticStruct());
		AITEST_EQUAL("Archetype should not change after removing sparse elements",
			EntityManager->GetArchetypeForEntity(Entity), ArchetypeWithTag);

		// Verify sparse elements removed
		AITEST_FALSE("Should not have sparse int fragment after removal",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
		AITEST_FALSE("Should not have sparse tag A after removal",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));

		// Verify regular elements still present
		AITEST_TRUE("Should still have regular float fragment",
			EntityManager->DoesEntityHaveElement(Entity, FTestFragment_Float::StaticStruct()));
		AITEST_TRUE("Should still have regular tag A",
			EntityManager->DoesEntityHaveElement(Entity, FTestTag_A::StaticStruct()));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_MixedRegularAndSparse, "System.Mass.Stress.Sparse.UnifiedAPI.MixedRegularAndSparse");


struct FSparseElements_UnifiedAPI_BatchOperations : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		constexpr int32 NumEntities = 10;
		TArray<FMassEntityHandle> Entities;
		EntityManager->BatchCreateEntities(IntsArchetype, NumEntities, Entities);

		// Add sparse fragments to all entities using unified API
		for (const FMassEntityHandle& Entity : Entities)
		{
			EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());
			EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());
		}

		// Verify all entities have sparse elements
		for (const FMassEntityHandle& Entity : Entities)
		{
			AITEST_TRUE("Each entity should have sparse fragment",
				EntityManager->DoesEntityHaveElement(Entity, FTestFragment_SparseInt::StaticStruct()));
			AITEST_TRUE("Each entity should have sparse tag",
				EntityManager->DoesEntityHaveElement(Entity, FTestTag_SparseA::StaticStruct()));
		}

		// Remove from half
		for (int32 i = 0; i < NumEntities / 2; ++i)
		{
			EntityManager->RemoveFragmentFromEntity(Entities[i], FTestFragment_SparseInt::StaticStruct());
			EntityManager->RemoveTagFromEntity(Entities[i], FTestTag_SparseA::StaticStruct());
		}

		// Verify first half don't have, second half still have
		for (int32 i = 0; i < NumEntities; ++i)
		{
			if (i < NumEntities / 2)
			{
				AITEST_FALSE("First half should not have sparse fragment",
					EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
				AITEST_FALSE("First half should not have sparse tag",
					EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));
			}
			else
			{
				AITEST_TRUE("Second half should have sparse fragment",
					EntityManager->DoesEntityHaveElement(Entities[i], FTestFragment_SparseInt::StaticStruct()));
				AITEST_TRUE("Second half should have sparse tag",
					EntityManager->DoesEntityHaveElement(Entities[i], FTestTag_SparseA::StaticStruct()));
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_BatchOperations, "System.Mass.Stress.Sparse.UnifiedAPI.BatchOperations");


struct FSparseElements_UnifiedAPI_RemoveNonExistent : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Try to remove sparse elements that were never added - should not crash
		EntityManager->RemoveFragmentFromEntity(Entity, FTestFragment_SparseInt::StaticStruct());
		EntityManager->RemoveTagFromEntity(Entity, FTestTag_SparseA::StaticStruct());
		EntityManager->RemoveElementFromEntity(Entity, FTestFragment_SparseFloat::StaticStruct());

		// Verify entity is still valid
		AITEST_TRUE("Entity should still be valid after removing non-existent sparse elements",
			EntityManager->IsEntityValid(Entity));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_RemoveNonExistent, "System.Mass.Stress.Sparse.UnifiedAPI.RemoveNonExistent");


struct FSparseElements_UnifiedAPI_AddDuplicate : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Add sparse fragment
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Set a value
		FMassEntityView EntityView(*EntityManager, Entity);
		FTestFragment_SparseInt* SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Should have sparse fragment", SparseFragment);
		SparseFragment->Value = 123;

		// Try to add the same sparse fragment again
		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		// Verify value is preserved (not reset)
		SparseFragment = EntityView.GetSparseFragmentDataPtr<FTestFragment_SparseInt>();
		AITEST_NOT_NULL("Should still have sparse fragment", SparseFragment);
		AITEST_EQUAL("Sparse fragment value should be preserved after duplicate add", SparseFragment->Value, 123);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_AddDuplicate, "System.Mass.Stress.Sparse.UnifiedAPI.AddDuplicate");


struct FSparseElements_UnifiedAPI_EntityView_HasFragment_Sparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
		FMassEntityView EntityView(*EntityManager, Entity);

		// Verify HasFragment works for sparse fragments
		AITEST_FALSE("HasFragment should return false for absent sparse fragment",
			EntityView.HasFragment(FTestFragment_SparseInt::StaticStruct()));

		EntityManager->AddFragmentToEntity(Entity, FTestFragment_SparseInt::StaticStruct());

		AITEST_TRUE("HasFragment should return true for present sparse fragment",
			EntityView.HasFragment(FTestFragment_SparseInt::StaticStruct()));

		// Template version
		AITEST_TRUE("Template HasFragment should work for sparse fragment",
			EntityView.HasFragment<FTestFragment_SparseInt>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_EntityView_HasFragment_Sparse, "System.Mass.Stress.Sparse.UnifiedAPI.EntityView.HasFragment");


struct FSparseElements_UnifiedAPI_EntityView_HasTag_Sparse : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);
		FMassEntityView EntityView(*EntityManager, Entity);

		// Verify HasTag works for sparse tags
		AITEST_FALSE("HasTag should return false for absent sparse tag",
			EntityView.HasTag(FTestTag_SparseA::StaticStruct()));

		EntityManager->AddTagToEntity(Entity, FTestTag_SparseA::StaticStruct());

		AITEST_TRUE("HasTag should return true for present sparse tag",
			EntityView.HasTag(FTestTag_SparseA::StaticStruct()));

		// Template version
		AITEST_TRUE("Template HasTag should work for sparse tag",
			EntityView.HasTag<FTestTag_SparseA>());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_EntityView_HasTag_Sparse, "System.Mass.Stress.Sparse.UnifiedAPI.EntityView.HasTag");


struct FSparseElements_UnifiedAPI_FragmentListOperations : FEntityTestBase
{
	virtual bool InstantTest() override
	{
		const FMassEntityHandle Entity = EntityManager->CreateEntity(IntsArchetype);

		// Create array of sparse fragment types for RemoveFragmentListFromEntity
		TArray<const UScriptStruct*> SparseFragmentTypes;
		SparseFragmentTypes.Add(FTestFragment_SparseInt::StaticStruct());
		SparseFragmentTypes.Add(FTestFragment_SparseFloat::StaticStruct());

		// Add sparse fragments first
		for (const UScriptStruct* Type : SparseFragmentTypes)
		{
			EntityManager->AddFragmentToEntity(Entity, Type);
		}

		// Verify all added
		AITEST_TRUE("Should have sparse int", EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entity));
		AITEST_TRUE("Should have sparse float", EntityManager->DoesEntityHaveElement<FTestFragment_SparseFloat>(Entity));

		// Remove using RemoveFragmentListFromEntity
		EntityManager->RemoveFragmentListFromEntity(Entity, SparseFragmentTypes);

		// Verify all removed
		AITEST_FALSE("Should not have sparse int after list removal",
			EntityManager->DoesEntityHaveElement<FTestFragment_SparseInt>(Entity));
		AITEST_FALSE("Should not have sparse float after list removal",
			EntityManager->DoesEntityHaveElement<FTestFragment_SparseFloat>(Entity));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FSparseElements_UnifiedAPI_FragmentListOperations, "System.Mass.Stress.Sparse.UnifiedAPI.FragmentListOperations");

} // namespace

#undef WITH_ENTITY_MOVING_AFFECTING_COMPOSITION

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
