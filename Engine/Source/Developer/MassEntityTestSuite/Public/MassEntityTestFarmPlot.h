// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "GameFramework/Actor.h"
#include "Engine/DataTable.h"
#include "MassEntityTypes.h"
#include "MassEntityManager.h"
#include "MassEntityQuery.h"
#include "MassCommandBuffer.h"
#include "MassProcessor.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTestFarmPlot.generated.h"

class UStaticMesh;
class UMaterialInterface;
class UHierarchicalInstancedStaticMeshComponent;

USTRUCT()
struct FFarmVisualDataRow : public FTableRowBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Farm)
	TObjectPtr<UStaticMesh> Mesh = nullptr;

	UPROPERTY(EditAnywhere, Category=Farm)
	TObjectPtr<UMaterialInterface> MaterialOverride = nullptr;
};

USTRUCT()
struct FFarmJustBecameReadyToHarvestTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FFarmReadyToHarvestTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FFarmGridCellData : public FMassFragment
{
	GENERATED_BODY()

	uint16 CellX = 0;
	uint16 CellY = 0;
};

USTRUCT()
struct FFarmWaterFragment : public FMassFragment
{
	GENERATED_BODY()

	float CurrentWater = 1.0f;
	float DeltaWaterPerSecond = -0.01f;
};

USTRUCT()
struct FFarmFlowerFragment : public FMassFragment
{
	GENERATED_BODY()
		
	uint32 NumBonusTicks = 0;
	uint16 FlowerType = 0;
};

USTRUCT()
struct FFarmCropFragment : public FMassFragment
{
	GENERATED_BODY()

	uint16 CropType = 0;
};


USTRUCT()
struct FFarmVisualFragment : public FMassFragment
{
	GENERATED_BODY()

	int32 InstanceIndex = -1;
	int32 HarvestIconIndex = -1;
	uint16 VisualType = 0;
};

USTRUCT()
struct FHarvestTimerFragment : public FMassFragment
{
	GENERATED_BODY()

	uint32 NumSecondsLeft = 15;
};

//////////////////////////////////////////////////////////////////////

UCLASS(abstract)
class UFarmProcessorBase : public UMassProcessor
{
	GENERATED_BODY()

public:
	UFarmProcessorBase();

protected:
	FMassEntityQuery EntityQuery;
};

//////////////////////////////////////////////////////////////////////

UCLASS()
class UFarmWaterProcessor : public UFarmProcessorBase
{
	GENERATED_BODY()

public:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

//////////////////////////////////////////////////////////////////////

UCLASS()
class UFarmHarvestTimerSystem_Flowers : public UFarmProcessorBase
{
	GENERATED_BODY()

	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

//////////////////////////////////////////////////////////////////////

UCLASS()
class UFarmHarvestTimerSystem_Crops : public UFarmProcessorBase
{
	GENERATED_BODY()

	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

//////////////////////////////////////////////////////////////////////

UCLASS()
class UFarmHarvestTimerExpired : public UFarmProcessorBase
{
	GENERATED_BODY()

	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

//////////////////////////////////////////////////////////////////////

UCLASS()
class UFarmHarvestTimerSetIcon : public UFarmProcessorBase
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TObjectPtr<UHierarchicalInstancedStaticMeshComponent> HarvestIconISMC;

	float GridCellWidth;
	float GridCellHeight;
	float HarvestIconHeight;
	float HarvestIconScale;

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
};

//////////////////////////////////////////////////////////////////////

UCLASS(config=Game)
class AMassEntityTestFarmPlot : public AActor
{
	GENERATED_BODY()

public:
	uint16 GridWidth = 40*7;
	uint16 GridHeight = 20*7;

	UPROPERTY(EditAnywhere, Category=Farm)
	float GridCellWidth = 150.0f;
	
	UPROPERTY(EditAnywhere, Category=Farm)
	float GridCellHeight = 150.0f;
	
	UPROPERTY(EditAnywhere, Category=Farm)
	float HarvestIconScale = 0.3f;
	
	TArray<FMassEntityHandle> PlantedSquares;

	
	UPROPERTY(EditAnywhere, Category=Farm)
	TArray<FFarmVisualDataRow> VisualDataTable;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UHierarchicalInstancedStaticMeshComponent>> VisualDataISMCs;

	float NextSecondTimer = 0.0f;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMassProcessor>> PerFrameSystems;

	UPROPERTY(Transient)
	TArray<TObjectPtr<UMassProcessor>> PerSecondSystems;

	// Indices into VisualDataTable for flowers
	UPROPERTY(EditAnywhere, Category=Farm)
	TArray<uint16> TestDataFlowerIndicies;

	// Indices into VisualDataTable for crops
	UPROPERTY(EditAnywhere, Category=Farm)
	TArray<uint16> TestDataCropIndicies;

	UPROPERTY(EditAnywhere, Category = Farm)
	uint32 VisualNearCullDistance = 1000;

	UPROPERTY(EditAnywhere, Category = Farm)
	uint32 VisualFarCullDistance = 1200;

	UPROPERTY(EditAnywhere, Category = Farm)
	uint32 IconNearCullDistance = 400;

	UPROPERTY(EditAnywhere, Category = Farm)
	uint32 IconFarCullDistance = 800;

private:
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, meta=(AllowPrivateAccess = "true"), Category=Farm)
	TObjectPtr<UHierarchicalInstancedStaticMeshComponent> HarvestIconISMC;

	TArray<int32> FreeHarvestIconIndicies;

	TSharedRef<FMassEntityManager> SharedEntityManager;

private:
	void AddItemToGrid(FMassEntityManager& InEntityManager, uint16 X, uint16 Y, FMassArchetypeHandle Archetype, uint16 VisualIndex);

public:
	AMassEntityTestFarmPlot();

	virtual void BeginPlay() override;
	virtual void TickActor(float DeltaTime, enum ELevelTick TickType, FActorTickFunction& ThisTickFunction) override;
};
