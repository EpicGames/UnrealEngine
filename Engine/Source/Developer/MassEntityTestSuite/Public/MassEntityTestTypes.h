// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassObserverProcessor.h"
#include "MassEntityQuery.h"
#include "MassEntityTypes.h"
#include "MassEntityManager.h"
#include "AITestsCommon.h"
#include "Math/RandomStream.h"
#include "Subsystems/EngineSubsystem.h"
#include "Subsystems/WorldSubsystem.h"
#include "Subsystems/LocalPlayerSubsystem.h"
#include "Subsystems/GameInstanceSubsystem.h"
#include "Misc/MTAccessDetector.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassProcessingPhaseManager.h"
#include "MassSubsystemBase.h"
#include "MassEntityTestTypes.generated.h"

#define UE_API MASSENTITYTESTSUITE_API

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
/** Works with checks too, but didn't want to make the name too lon */
#define MASS_SCOPED_ENSURE_TEST(PartialTextToExpect, NumOccurrences) \
	GetTestRunner().AddExpectedErrorPlain(PartialTextToExpect, EAutomationExpectedErrorFlags::Contains, NumOccurrences); \
	ON_SCOPE_EXIT \
	{ \
		--UE::Mass::Test::TestsInProgress; \
		FAutomationTestExecutionInfo ExecutionInfo; \
		GetTestRunner().GetExecutionInfo(ExecutionInfo); \
		if (ExecutionInfo.GetWarningTotal() + ExecutionInfo.GetErrorTotal() > 0) \
		{ \
			UE::AITestSuite::ConditionallyBreakOnTestFail(); \
		} \
	}; \
	++UE::Mass::Test::TestsInProgress
#else
#define MASS_SCOPED_ENSURE_TEST(PartialTextToExpect, NumOccurrences) \
	GetTestRunner().AddExpectedErrorPlain(PartialTextToExpect, EAutomationExpectedErrorFlags::Contains, NumOccurrences); \
	ON_SCOPE_EXIT \
	{ \
		FAutomationTestExecutionInfo ExecutionInfo; \
		GetTestRunner().GetExecutionInfo(ExecutionInfo); \
		if (ExecutionInfo.GetWarningTotal() + ExecutionInfo.GetErrorTotal() > 0) \
		{ \
			UE::AITestSuite::ConditionallyBreakOnTestFail(); \
		} \
	}
#endif 

struct FMassEntityManager;
struct FMassProcessingPhaseManager;
namespace UE::Mass::Testing
{
	struct FMassTestProcessingPhaseManager;
}

USTRUCT()
struct FTestFragment_EntityHandle : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	FMassEntityHandle Value;

	FTestFragment_EntityHandle(const FMassEntityHandle InValue = FMassEntityHandle()) : Value(InValue) {}
};

USTRUCT()
struct FTestFragment_Float : public FMassFragment
{
	GENERATED_BODY()
	float Value = 0;

	FTestFragment_Float(const float InValue = 0.f) : Value(InValue) {}
};

USTRUCT()
struct FTestFragment_Int : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestFragment_Int(const int32 InValue = 0) : Value(InValue) {}

	static constexpr int32 TestIntValue = 123456;
};

USTRUCT()
struct FTestFragment_Bool : public FMassFragment
{
	GENERATED_BODY()
	bool bValue = false;

	FTestFragment_Bool(const bool bInValue = false) : bValue(bInValue) {}
};

USTRUCT()
struct FTestFragment_Large : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 Value[64];

	FTestFragment_Large(uint8 Fill = 0)
	{
		FMemory::Memset(Value, Fill, 64);
	}
};

USTRUCT()
struct FTestFragment_Array : public FMassFragment
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> Value;

	FTestFragment_Array(uint8 Num = 0)
	{
		Value.Reserve(Num);
	}
};

template<>
struct TMassFragmentTraits<FTestFragment_Array> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

USTRUCT()
struct FFragmentWithSharedPtr : public FMassFragment
{
	GENERATED_BODY()

	TSharedPtr<int32> Data;

	FFragmentWithSharedPtr() = default;
	FFragmentWithSharedPtr(TSharedPtr<int32>& InData)
		: Data(InData)
	{}
};

template<>
struct TMassFragmentTraits<FFragmentWithSharedPtr> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

USTRUCT()
struct FTestChunkFragment_Int : public FMassChunkFragment
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestChunkFragment_Int(const int32 InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestChunkFragment_Float : public FMassChunkFragment
{
	GENERATED_BODY()

	UPROPERTY()
	float Value = 0.f;

	FTestChunkFragment_Float(const float InValue = 0.f) : Value(InValue) {}
};

USTRUCT()
struct FTestSharedFragment_Int : public FMassSharedFragment
{
	using FValueType = int32;

	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestSharedFragment_Int(const int32 InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestSharedFragment_Array : public FMassSharedFragment
{
	using FValueType = TArray<int32>;

	GENERATED_BODY()

	UPROPERTY()
	TArray<int32> Value;

	FTestSharedFragment_Array() = default;
	FTestSharedFragment_Array(const TArray<int32>&& InValue) : Value(InValue) {}
};

template<>
struct TMassSharedFragmentTraits<FTestSharedFragment_Int> final
{
	enum
	{
		GameThreadOnly = true
	};
};

USTRUCT()
struct FTestConstSharedFragment_Int : public FMassConstSharedFragment
{
	using FValueType = int32;

	GENERATED_BODY()

	UPROPERTY()
	int32 Value = 0;

	FTestConstSharedFragment_Int(const int32 InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestSharedFragment_Float : public FMassSharedFragment
{
	using FValueType = float;

	GENERATED_BODY()

	UPROPERTY()
	float Value = 0.f;

	FTestSharedFragment_Float(const float InValue = 0) : Value(InValue) {}
};

USTRUCT()
struct FTestConstSharedFragment_Float : public FMassConstSharedFragment
{
	using FValueType = float;

	GENERATED_BODY()

	UPROPERTY()
	float Value = 0.f;

	FTestConstSharedFragment_Float(const float InValue = 0) : Value(InValue) {}
};

/** @todo rename to FTestTag */
USTRUCT()
struct FTestFragment_Tag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_A : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_B : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_C : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_D : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestFragment_SparseInt : public FMassSparseFragment
{
	GENERATED_BODY()

	FTestFragment_SparseInt(const int32 InValue = 0)
		: Value(InValue)
	{
	}

	int32 Value = 0;
};

USTRUCT()
struct FTestFragment_SparseFloat : public FMassSparseFragment
{
	GENERATED_BODY()

	FTestFragment_SparseFloat(const float InValue = 0.f)
		: Value(InValue)
	{
	}

	float Value = 0.f;
};

USTRUCT()
struct FTestTag_SparseA : public FMassSparseTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestTag_SparseB : public FMassSparseTag
{
	GENERATED_BODY()
};


UCLASS(MinimalAPI)
class UMassTestProcessorBase : public UMassProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassTestProcessorBase();
	FMassProcessorExecutionOrder& GetMutableExecutionOrder() { return ExecutionOrder; }
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const { return false; }
	/** 
	 * Leaving the implementation empty since it's up to the child classes and specific use-cases
	 * to determine the actual requirements for EntityQuery.
	 */
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override {}

	using FExecutionFunction = TFunction<void(FMassEntityManager& EntityManager, FMassExecutionContext& Context)>;
	FExecutionFunction ExecutionFunction;

	/** 
	 * By default ExecutionFunction is configured to pass this function over to EntityQuery.ForEachEntityChunk call. 
	 * Note that this function won't be used if you override ExecutionFunction's default value.
	 */
	FMassExecuteFunction ForEachEntityChunkExecutionFunction;

	void SetShouldAllowMultipleInstances(const bool bInShouldAllowDuplicated) { bAllowMultipleInstances = bInShouldAllowDuplicated; }

	UE_API void SetUseParallelForEachEntityChunk(bool bEnable);
	UE_API void TestExecute(TSharedPtr<FMassEntityManager>& EntityManager);

	FMassEntityCreationRequirements& GetEntityCreationRequirements()
	{
		return ProcessorEntityCreationRequirements;
	}

	/** public on purpose, this is a test processor, no worries about access*/
	FMassEntityQuery EntityQuery;
};

UCLASS(MinimalAPI)
class UMassTestProcessor_A : public UMassTestProcessorBase
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassTestProcessor_B : public UMassTestProcessorBase
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassTestProcessor_C : public UMassTestProcessorBase
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassTestProcessor_D : public UMassTestProcessorBase
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassTestProcessor_E : public UMassTestProcessorBase
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UMassTestProcessor_F : public UMassTestProcessorBase
{
	GENERATED_BODY()
};

UCLASS()
class UMassTestProcessor_Floats : public UMassTestProcessorBase
{
	GENERATED_BODY()
public:
	TArrayView<FTestFragment_Float> Floats;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

UCLASS()
class UMassTestProcessor_Ints : public UMassTestProcessorBase
{
	GENERATED_BODY()
public:
	TArrayView<FTestFragment_Int> Ints;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

UCLASS()
class UMassTestProcessor_FloatsInts : public UMassTestProcessorBase
{
	GENERATED_BODY()
public:
	TArrayView<FTestFragment_Float> Floats;
	TArrayView<FTestFragment_Int> Ints;
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};

template<typename T>
T* NewTestProcessor(TSharedPtr<FMassEntityManager> EntityManager)
{
	check(EntityManager);
	T* NewProcessor = NewObject<T>();
	CA_ASSUME(NewProcessor);
	NewProcessor->CallInitialize(GetTransientPackage(), EntityManager.ToSharedRef());
	return NewProcessor;
}

UCLASS(MinimalAPI)
class UMassTestStaticCounterProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassTestStaticCounterProcessor();
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override
	{
		++StaticCounter;
	}
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override {}
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const override { return false; }

	static UE_API int StaticCounter;
};

UCLASS()
class UMassTestProcessorAutoExecuteQuery : public UMassProcessor
{
	GENERATED_BODY()
public:
	UMassTestProcessorAutoExecuteQuery();
	void SetAutoExecuteQuery(TSharedPtr<UE::Mass::FQueryExecutor> InAutoExecuteQuery)
	{
		AutoExecuteQuery = InAutoExecuteQuery;
	}
	FMassEntityQuery EntityQuery{ *this };
};

UCLASS()
class UMassTestProcessorAutoExecuteQueryComparison : public UMassTestProcessorBase
{
	GENERATED_BODY()
public:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	FMassEntityQuery EntityQuery{ *this };
};

UCLASS()
class UMassTestProcessorAutoExecuteQueryComparison_Parallel : public UMassTestProcessorBase
{
	GENERATED_BODY()
public:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>&) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	FMassEntityQuery EntityQuery{ *this };
};

struct FExecutionTestBase : FAITestBase
{
	TSharedPtr<FMassEntityManager> EntityManager;
	bool bMakeWorldEntityManagersOwner = true;

	UE_API virtual bool SetUp() override;
	UE_API virtual void TearDown() override;
};

struct FEntityTestBase : FExecutionTestBase
{
	FMassArchetypeHandle EmptyArchetype;
	FMassArchetypeHandle FloatsArchetype;
	FMassArchetypeHandle IntsArchetype;
	FMassArchetypeHandle FloatsIntsArchetype;
	FMassArchetypeHandle LargeArchetype;

	FInstancedStruct InstanceInt;

	UE_API virtual bool SetUp() override;
};


struct FProcessingPhasesTestBase : FEntityTestBase
{
	using Super = FEntityTestBase;

	TSharedPtr<UE::Mass::Testing::FMassTestProcessingPhaseManager> PhaseManager;
	FMassProcessingPhaseConfig PhasesConfig[int(EMassProcessingPhase::MAX)];
	int32 TickIndex = -1;
	FGraphEventRef CompletionEvent;
	float DeltaTime = 1.f / 30;
	UWorld* World = nullptr;

	UE_API FProcessingPhasesTestBase();
	UE_API virtual bool SetUp() override;
	UE_API virtual bool Update() override;
	UE_API virtual void TearDown() override;
	UE_API virtual void VerifyLatentResults() override;
	virtual bool PopulatePhasesConfig() = 0;
};

template<typename T>
void ShuffleDataWithRandomStream(FRandomStream& Rand, TArray<T>& Data)
{
	for (int i = 0; i < Data.Num(); ++i)
	{
		const int32 NewIndex = Rand.RandRange(0, Data.Num() - 1);
		Data.Swap(i, NewIndex);
	}
}

UCLASS()
class UMassTestWorldSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()
public:
	void Write(int32 InNumber);
	int32 Read() const;

private:
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(AccessDetector);
	int Number = 0;
};

template<>
struct TMassExternalSubsystemTraits<UMassTestWorldSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

UCLASS()
class UMassTestParallelSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassTestParallelSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = true,
	};
};

UCLASS()
class UMassTestEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassTestEngineSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

UCLASS()
class UMassTestLocalPlayerSubsystem : public ULocalPlayerSubsystem
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassTestLocalPlayerSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
}; 

UCLASS()
class UMassTestGameInstanceSubsystem : public UGameInstanceSubsystem
{
	GENERATED_BODY()
};

template<>
struct TMassExternalSubsystemTraits<UMassTestGameInstanceSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

namespace UE::Mass::Testing
{
/** Test-time TaskGraph task for triggering processing phases. */
struct FMassTestPhaseTickTask
{
	FMassTestPhaseTickTask(const TSharedRef<FMassProcessingPhaseManager>& InPhaseManager, const EMassProcessingPhase InPhase, const float InDeltaTime);

	static TStatId GetStatId();
	static ENamedThreads::Type GetDesiredThread();
	static ESubsequentsMode::Type GetSubsequentsMode();

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);

private:
	const TSharedRef<FMassProcessingPhaseManager> PhaseManager;
	const EMassProcessingPhase Phase = EMassProcessingPhase::MAX;
	const float DeltaTime = 0.f;
};

/** The main point of this FMassProcessingPhaseManager extension is to disable world-based ticking, even if a world is available. */
struct FMassTestProcessingPhaseManager : public FMassProcessingPhaseManager
{
	void Start(const TSharedPtr<FMassEntityManager>& InEntityManager);
	void OnNewArchetype(const FMassArchetypeHandle& NewArchetype);
};

} // namespace UE::Mass::Testing

/**
 * Minimal concrete multi-type observer processor for unit tests.
 * Counts invocations of Execute() to verify deduplication guarantees.
 * bAutoRegisterWithObserverRegistry is set to false in the constructor so that the CDO
 * produced during module init does not attempt to contact UMassObserverRegistry.
 */
UCLASS(MinimalAPI)
class UMassTestCompositeObserverProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UMassTestCompositeObserverProcessor()
	{
		bAutoRegisterWithObserverRegistry = false;
		QueryBasedPruning = EMassQueryBasedPruning::Never;
	}

	int32 ExecuteCallCount = 0;

	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override
	{
		++ExecuteCallCount;
	}

	/** Sets the observed types list directly for test configuration. */
	void SetObservedTypesForTest(std::initializer_list<const UScriptStruct*> Types)
	{
		ObservedTypes.Reset(static_cast<int32>(Types.size()));
		for (const UScriptStruct* T : Types)
		{
			ObservedTypes.Add(T);
		}
	}

	/** Sets the observed operation flags directly for test configuration. */
	void SetObservedOperationsForTest(const EMassObservedOperationFlags InFlags)
	{
		ObservedOperations = InFlags;
	}
};

#undef UE_API
